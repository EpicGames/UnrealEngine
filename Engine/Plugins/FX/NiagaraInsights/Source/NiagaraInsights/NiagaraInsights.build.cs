// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NiagaraInsights : ModuleRules
	{
		public NiagaraInsights(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"SlateCore",
				"Slate",
				"TraceLog",
				"TraceAnalysis",
				"TraceServices",
				"TraceInsights",
				"TraceInsightsCore",
				"InputCore"
			});
		}
	}
}

