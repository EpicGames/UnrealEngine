// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraAnalyzer.h"
#include "NiagaraProvider.h"

#include "TraceServices/Model/AnalysisSession.h"

#define LOCTEXT_NAMESPACE "NiagaraProvider"

namespace UE::NiagaraInsights
{

static FStringView GetEventString(const Trace::IAnalyzer::FEventData& EventData, const ANSICHAR* FieldName)
{
	FStringView StringView;
	EventData.GetString(FieldName, StringView);
	return StringView;
}

FNiagaraAnalyzer::FNiagaraAnalyzer(TraceServices::IAnalysisSession& InSession, FNiagaraProvider& InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
}

void FNiagaraAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;
	
	Builder.RouteEvent(RouteId_SystemPerformance_GT, "NiagaraTrace", "SystemPerformance_GT");
	Builder.RouteEvent(RouteId_SystemPerformance_RT, "NiagaraTrace", "SystemPerformance_RT");

	Builder.RouteEvent(RouteId_ComponentActivate,	"NiagaraTrace", "ComponentActivate");
	Builder.RouteEvent(RouteId_ComponentDeactivate,	"NiagaraTrace", "ComponentDeactivate");
	Builder.RouteEvent(RouteId_ComponentComplete,	"NiagaraTrace", "ComponentComplete");

	Builder.RouteEvent(RouteId_DataChannelPublish, "NiagaraTrace", "DataChannelPublish");
	Builder.RouteEvent(RouteId_DataChannelWrite, "NiagaraTrace", "DataChannelWrite");
}

bool FNiagaraAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	//LLM_SCOPE_BYNAME(TEXT("Insights/FNiagaraAnalyzer"));

	const Trace::IAnalyzer::FEventData& EventData = Context.EventData;
	const double EventTime = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("EventTime"));

	TraceServices::FAnalysisSessionEditScope _(Session);
	switch (RouteId)
	{
		case RouteId_SystemPerformance_GT:
		{
			Provider.AddSystemPerformance_GT(
				EventTime,
				GetEventString(EventData, "SystemName"),
				EventData.GetValue<uint64>("NumInstances"),
				Context.EventTime.AsSecondsAbsolute(EventData.GetValue<uint64>("TickGameThreadCycles")),
				Context.EventTime.AsSecondsAbsolute(EventData.GetValue<uint64>("TickConcurrentCycles")),
				Context.EventTime.AsSecondsAbsolute(EventData.GetValue<uint64>("FinalizeCycles")),
				Context.EventTime.AsSecondsAbsolute(EventData.GetValue<uint64>("EndOfFrameCycles")),
				Context.EventTime.AsSecondsAbsolute(EventData.GetValue<uint64>("ActivationCycles")),
				Context.EventTime.AsSecondsAbsolute(EventData.GetValue<uint64>("WaitCycles")),
				EventData.GetValue<uint64>("MemoryBytes")
			);
			break;
		}

		case RouteId_SystemPerformance_RT:
		{
			Provider.AddSystemPerformance_RT(
				EventTime,
				GetEventString(EventData, "SystemName"),
				EventData.GetValue<uint64>("NumInstances"),
				Context.EventTime.AsSecondsAbsolute(EventData.GetValue<uint64>("RenderUpdateCycles")),
				Context.EventTime.AsSecondsAbsolute(EventData.GetValue<uint64>("GetDynamicMeshElementsCycles")),
				EventData.GetValue<uint64>("GpuNumInstances"),
				EventData.GetValue<uint64>("GpuTotalMicroseconds")
			);
			break;
		}

		case RouteId_ComponentActivate:
		{
			Provider.AddComponentActivate(
				EventTime,
				GetEventString(EventData, "ComponentName"),
				EventData.GetValue<bool>("bReset"),
				EventData.GetValue<bool>("bIsScalabilityCull"),
				EventData.GetValue<bool>("bAwaitingActivationDueToNotReady")
			);
			break;
		}

		case RouteId_ComponentDeactivate:
		{
			Provider.AddComponentDeactivate(
				EventTime,
				GetEventString(EventData, "ComponentName"),
				EventData.GetValue<bool>("bImmediate"),
				EventData.GetValue<bool>("bIsScalabilityCull")
			);
			break;
		}

		case RouteId_ComponentComplete:
		{
			Provider.AddComponentComplete(
				EventTime,
				GetEventString(EventData, "ComponentName"),
				EventData.GetValue<bool>("bExternalCompletion")
			);
			break;
		}

		case RouteId_DataChannelPublish:
		{
			Provider.AddDataChannelPublish(
				EventTime,
				GetEventString(EventData, "SourceName"),
				EventData.GetValue<bool>("bGpuRequest"),
				EventData.GetValue<bool>("bVisibleToGame"),
				EventData.GetValue<bool>("bVisibleToCPUSims"),
				EventData.GetValue<bool>("bVisibleToGPUSims"),
				EventData.GetValue<uint32>("NumInstances"),
				EventData.GetValue<uint32>("NumInstanceAllocated")
			);
			break;
		}

		case RouteId_DataChannelWrite:
		{
			Provider.AddDataChannelWrite(
				EventTime,
				GetEventString(EventData, "DataChannelName"),
				GetEventString(EventData, "SourceName"),
				EventData.GetValue<int32>("NumInstances"),
				EventData.GetValue<bool>("bVisibleToGame"),
				EventData.GetValue<bool>("bVisibleToCPU"),
				EventData.GetValue<bool>("bVisibleToGPU")
			);
			break;
		}
	}
	return true;
}

} //namespace UE::NiagaraInsights

#undef LOCTEXT_NAMESPACE
