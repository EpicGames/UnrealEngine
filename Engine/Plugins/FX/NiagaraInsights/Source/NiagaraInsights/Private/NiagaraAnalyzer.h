// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

namespace TraceServices { class IAnalysisSession; }

namespace UE::NiagaraInsights
{

class FNiagaraProvider;

class FNiagaraAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FNiagaraAnalyzer(TraceServices::IAnalysisSession& InSession, FNiagaraProvider& InProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override {}
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_SystemPerformance_GT,
		RouteId_SystemPerformance_RT,

		RouteId_ComponentActivate,
		RouteId_ComponentDeactivate,
		RouteId_ComponentComplete,

		RouteId_DataChannelPublish,
		RouteId_DataChannelWrite,
	};

	TraceServices::IAnalysisSession&	Session;
	FNiagaraProvider&					Provider;
};

} //namespace UE::NiagaraInsights
