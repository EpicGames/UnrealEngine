// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelTrack.h"

#include "NiagaraInsightsCommon.h"
#include "NiagaraProvider.h"
#include "NiagaraTimingViewSession.h"

#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "Insights/ViewModels/GraphTrackEvent.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#define LOCTEXT_NAMESPACE "NiagaraDataChannelTrack"

namespace UE::NiagaraInsights
{

INSIGHTS_IMPLEMENT_RTTI(FNiagaraDataChannelTrack)

FNiagaraDataChannelTrack::FNiagaraDataChannelTrack(const FNiagaraTimingViewSession& InSharedData)
	: FGraphTrack(LOCTEXT("NiagaraDataChannel", "Niagara Data Channel").ToString())
	, SharedData(InSharedData)
{
	constexpr double NumEventTypes = 2.0;

	EnabledOptions = //EGraphOptions::ShowDebugInfo |
		EGraphOptions::ShowPoints |
		EGraphOptions::ShowLines |
		EGraphOptions::ShowPolygon |
		EGraphOptions::ShowVerticalAxisGrid |
		EGraphOptions::ShowHeader;

	SetValidLocations(ETimingTrackLocation::Scrollable | ETimingTrackLocation::TopDocked | ETimingTrackLocation::BottomDocked);
	SetHeight(NumEventTypes * 10.0);

	TSharedRef<FGraphSeries> GraphSeries = MakeShared<FGraphSeries>();
	GraphSeries->SetName(TEXT("Events"));
	GraphSeries->SetColor(Common::Color_MemoryCostTotal);
	GraphSeries->SetBaselineY(GetHeight());
	GraphSeries->SetScaleY(GetHeight() / (NumEventTypes + 1.0));
	GraphSeries->EnableAutoZoom();
	GetSeries().Add(GraphSeries);
}

void FNiagaraDataChannelTrack::Update(const ITimingTrackUpdateContext& Context)
{
	Super::Update(Context);

	const FNiagaraProvider* Provider = SharedData.GetAnalysisSession().ReadProvider<FNiagaraProvider>(FNiagaraProvider::GetProviderName());
	if (Provider == nullptr)
	{
		return;
	}

	const FTimingTrackViewport& Viewport = Context.GetViewport();

	const TSharedPtr<FGraphSeries> GraphSeries = GetSeries()[0];
	if (GraphSeries->IsVisible())
	{
		FGraphTrackBuilder Builder(*this, *GraphSeries, Viewport);
		Provider->EnumerateDataChannelEvent(
			Viewport.GetStartTime(),
			Viewport.GetEndTime(),
			[&Builder, &GraphSeries](const FDataChannelEvent& DataChannelEvent)
			{
				const double Offset = double(DataChannelEvent.Payload.GetIndex()) + 0.5;
				Builder.AddEvent(DataChannelEvent.Time, 0.0, Offset, false);
			}
		);
	}
}

void FNiagaraDataChannelTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	if (!InTooltipEvent.CheckTrack(this) || !InTooltipEvent.Is<FGraphTrackEvent>())
	{
		return;
	}

	const FGraphTrackEvent& GraphEvent = InTooltipEvent.As<FGraphTrackEvent>();
	const TSharedRef<const FGraphSeries> HoveredSeries = GraphEvent.GetSeries();

	const FNiagaraProvider* Provider = SharedData.GetAnalysisSession().ReadProvider<FNiagaraProvider>(FNiagaraProvider::GetProviderName());
	if (Provider == nullptr)
	{
		return;
	}

	InOutTooltip.ResetContent();
	InOutTooltip.AddTitle(HoveredSeries->GetName().ToString(), HoveredSeries->GetColor());

	const double EventTime = GraphEvent.GetStartTime();
	constexpr double TimeTolerance = 1e-9;

	Provider->EnumerateDataChannelEvent(
		EventTime - TimeTolerance,
		EventTime + TimeTolerance,
		[&InOutTooltip](const FDataChannelEvent& DataChannelEvent)
		{
			switch (DataChannelEvent.Payload.GetIndex())
			{
				case 0:
				{
					const FDataChannelEvent::FPublish& Payload = DataChannelEvent.Payload.Get<FDataChannelEvent::FPublish>();
					InOutTooltip.AddNameValueTextLine(TEXT("Publish"),				TEXT(""));
					InOutTooltip.AddNameValueTextLine(TEXT("SourceName"),			Payload.SourceName.IsEmpty() ? TEXT("<No Data>") : Payload.SourceName);
					InOutTooltip.AddNameValueTextLine(TEXT("bGpuRequest"),			LexToString(Payload.bGpuRequest));
					InOutTooltip.AddNameValueTextLine(TEXT("bVisibleToGame"),		LexToString(Payload.bVisibleToGame));
					InOutTooltip.AddNameValueTextLine(TEXT("bVisibleToCPUSims"),	LexToString(Payload.bVisibleToCPUSims));
					InOutTooltip.AddNameValueTextLine(TEXT("bVisibleToGPUSims"),	LexToString(Payload.bVisibleToGPUSims));
					InOutTooltip.AddNameValueTextLine(TEXT("NumInstances"),			LexToString(Payload.NumInstances));
					InOutTooltip.AddNameValueTextLine(TEXT("NumInstanceAllocated"),	LexToString(Payload.NumInstanceAllocated));
					break;
				}

				case 1:
				{
					const FDataChannelEvent::FWrite& Payload = DataChannelEvent.Payload.Get<FDataChannelEvent::FWrite>();
					InOutTooltip.AddNameValueTextLine(TEXT("Write"),				TEXT(""));
					InOutTooltip.AddNameValueTextLine(TEXT("DataChannelName"),		Payload.DataChannelName);
					InOutTooltip.AddNameValueTextLine(TEXT("SourceName"),			Payload.SourceName.IsEmpty() ? TEXT("<No Data>") : *Payload.SourceName);
					InOutTooltip.AddNameValueTextLine(TEXT("NumInstances"),			LexToString(Payload.NumInstances));
					InOutTooltip.AddNameValueTextLine(TEXT("bVisibleToGame"),		LexToString(Payload.bVisibleToGame));
					InOutTooltip.AddNameValueTextLine(TEXT("bVisibleToCPU"),		LexToString(Payload.bVisibleToCPU));
					InOutTooltip.AddNameValueTextLine(TEXT("bVisibleToGPU"),		LexToString(Payload.bVisibleToGPU));
					break;
				}
			}
		}
	);

	InOutTooltip.UpdateLayout();
}

} //namespace UE::NiagaraInsights

#undef LOCTEXT_NAMESPACE
