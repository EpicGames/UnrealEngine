// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ViewModels/GraphTrack.h"

namespace UE::NiagaraInsights
{

class FNiagaraTimingViewSession;

class FNiagaraDataChannelTrack : public FGraphTrack
{
	using Super = FGraphTrack;

	INSIGHTS_DECLARE_RTTI(FNiagaraDataChannelTrack, FGraphTrack)

public:
	explicit FNiagaraDataChannelTrack(const FNiagaraTimingViewSession& InSharedData);

	// BEGIN: FGraphTrack interface
	virtual void Update(const ITimingTrackUpdateContext& Context) override;
	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;
	// END: FGraphTrack interface

private:
	const FNiagaraTimingViewSession& SharedData;
};

} //namespace UE::NiagaraInsights
