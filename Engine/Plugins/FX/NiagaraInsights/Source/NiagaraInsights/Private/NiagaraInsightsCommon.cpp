// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraInsightsCommon.h"

namespace UE::NiagaraInsights::Common
{

const FLinearColor Color_InstanceCount(0.10f, 0.45f, 0.80f, 1.f);	// Blue
const FLinearColor Color_GTCostTotal(0.85f, 0.25f, 0.15f, 1.f);		// Red
const FLinearColor Color_GTCostPerInst(1.00f, 0.55f, 0.15f, 1.f);	// Orange
const FLinearColor Color_RTCostTotal(0.15f, 0.70f, 0.40f, 1.f);		// Green
const FLinearColor Color_RTCostPerInst(0.60f, 0.85f, 0.30f, 1.f);	// Yellow-green
const FLinearColor Color_GpuCostTotal(0.65f, 0.25f, 0.85f, 1.f);	// Purple
const FLinearColor Color_MemoryCostTotal(0.10f, 0.45f, 0.80f, 1.f);	// Purple

} // namespace UE::NiagaraInsights
