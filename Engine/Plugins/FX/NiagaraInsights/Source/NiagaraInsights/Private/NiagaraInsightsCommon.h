// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::NiagaraInsights::Common
{

extern const FLinearColor Color_InstanceCount;
extern const FLinearColor Color_GTCostTotal;
extern const FLinearColor Color_GTCostPerInst;
extern const FLinearColor Color_RTCostTotal;
extern const FLinearColor Color_RTCostPerInst;
extern const FLinearColor Color_GpuCostTotal;
extern const FLinearColor Color_MemoryCostTotal;

} // namespace UE::NiagaraInsights::Common
