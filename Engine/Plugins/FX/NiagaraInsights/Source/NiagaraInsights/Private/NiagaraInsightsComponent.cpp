// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraInsightsComponent.h"
#include "Widgets/SNiagaraRangeStatsView.h"

#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "NiagaraInsightsComponent"

namespace UE::NiagaraInsights
{

const FName FNiagaraInsightsTabs::RangeStatsViewID("NiagaraRangeStats");

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNiagaraInsightsComponent::Initialize(IUnrealInsightsModule& InsightsModule)
{
	// Nothing needed at init time; tab registration happens via RegisterMajorTabs.
}

void FNiagaraInsightsComponent::Shutdown()
{
}

void FNiagaraInsightsComponent::RegisterMajorTabs(IUnrealInsightsModule& InsightsModule)
{
	// Extend the Timing Profiler major tab to include our minor tab.
	FOnRegisterMajorTabExtensions& Delegate = InsightsModule.OnRegisterMajorTabExtension(FInsightsManagerTabs::TimingProfilerTabId);

	ExtensionDelegateHandle = Delegate.AddRaw(this, &FNiagaraInsightsComponent::OnRegisterTimingProfilerTabExtensions);
}

void FNiagaraInsightsComponent::UnregisterMajorTabs()
{
	if (ExtensionDelegateHandle.IsValid() && FModuleManager::Get().IsModuleLoaded("TraceInsights"))
	{
		IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		FOnRegisterMajorTabExtensions* Delegate = InsightsModule.FindMajorTabLayoutExtension(FInsightsManagerTabs::TimingProfilerTabId);
		if (Delegate)
		{
			Delegate->Remove(ExtensionDelegateHandle);
		}
	}
	ExtensionDelegateHandle.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNiagaraInsightsComponent::OnRegisterTimingProfilerTabExtensions(FInsightsMajorTabExtender& Extender)
{
	// Register the minor tab spawner with the Timing Profiler's tab manager.
	FMinorTabConfig& TabConfig = Extender.AddMinorTabConfig();
	TabConfig.TabId      = FNiagaraInsightsTabs::RangeStatsViewID;
	TabConfig.TabLabel   = LOCTEXT("NiagaraStatsTabTitle", "Niagara Stats");
	TabConfig.TabTooltip = LOCTEXT("NiagaraStatsTabTooltip",
		"Shows top-5 Niagara systems by various metrics for the selected time range.");
	TabConfig.OnSpawnTab = FOnSpawnTab::CreateRaw(this, &FNiagaraInsightsComponent::SpawnTab_RangeStatsView);

	// Place the tab after the existing "Timers" tab in the Timing Profiler layout.
	// The user can also drag it anywhere in the Timing Profiler window.
	Extender.GetLayoutExtender().ExtendLayout(
		FTimingProfilerTabs::TimersID,
		ELayoutExtensionPosition::After,
		FTabManager::FTab(FNiagaraInsightsTabs::RangeStatsViewID, ETabState::OpenedTab)
	);
}

TSharedRef<SDockTab> FNiagaraInsightsComponent::SpawnTab_RangeStatsView(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SNew(SNiagaraRangeStatsView)
		];
}

} // namespace UE::NiagaraInsights

#undef LOCTEXT_NAMESPACE
