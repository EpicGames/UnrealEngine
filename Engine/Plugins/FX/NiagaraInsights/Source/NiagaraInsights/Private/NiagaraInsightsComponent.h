// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/IUnrealInsightsModule.h"

namespace UE::NiagaraInsights
{

/** Tab IDs for the Niagara Insights minor tabs inside the Timing Profiler. */
struct FNiagaraInsightsTabs
{
	static const FName RangeStatsViewID;
};

/**
 * IInsightsComponent that adds the Niagara Range Stats minor tab into the Timing Profiler.
 * The tab shows top-5 offenders by instance count, GT cost, GT cost/instance, RT cost,
 * RT cost/instance, and GPU cost for the currently selected time range.
 */
class FNiagaraInsightsComponent : public IInsightsComponent
{
public:
	//~ Begin IInsightsComponent interface
	virtual void Initialize(IUnrealInsightsModule& InsightsModule) override;
	virtual void Shutdown() override;
	virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override;
	virtual void UnregisterMajorTabs() override;
	//~ End IInsightsComponent interface

private:
	void OnRegisterTimingProfilerTabExtensions(FInsightsMajorTabExtender& Extender);

	TSharedRef<SDockTab> SpawnTab_RangeStatsView(const FSpawnTabArgs& Args);

	FDelegateHandle ExtensionDelegateHandle;
};

} // namespace UE::NiagaraInsights
