// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraInsightsModule.h"
#include "NiagaraInsightsComponent.h"

#include "Features/IModularFeatures.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Modules/ModuleManager.h"

namespace UE::NiagaraInsights
{

void FNiagaraInsightsModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
	IModularFeatures::Get().RegisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);

	// Try to register with TraceInsights now; if it isn't loaded yet, listen for when it loads.
	TryRegisterInsightsComponent();
	if (!InsightsComponent.IsValid())
	{
		ModuleChangedHandle = FModuleManager::Get().OnModulesChanged().AddRaw(
			this, &FNiagaraInsightsModule::OnModulesChanged);
	}
}

void FNiagaraInsightsModule::TryRegisterInsightsComponent()
{
	if (InsightsComponent.IsValid())
	{
		return;
	}
	if (!FModuleManager::Get().IsModuleLoaded("TraceInsights"))
	{
		return;
	}
	IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	InsightsComponent = MakeShared<FNiagaraInsightsComponent>();
	InsightsModule.RegisterComponent(InsightsComponent);
}

void FNiagaraInsightsModule::OnModulesChanged(FName ModuleName, EModuleChangeReason Reason)
{
	if (ModuleName == "TraceInsights" && Reason == EModuleChangeReason::ModuleLoaded)
	{
		TryRegisterInsightsComponent();
		// No longer need to listen once registered.
		FModuleManager::Get().OnModulesChanged().Remove(ModuleChangedHandle);
		ModuleChangedHandle.Reset();
	}
}

void FNiagaraInsightsModule::ShutdownModule()
{
	if (ModuleChangedHandle.IsValid())
	{
		FModuleManager::Get().OnModulesChanged().Remove(ModuleChangedHandle);
		ModuleChangedHandle.Reset();
	}

	if (InsightsComponent.IsValid() && FModuleManager::Get().IsModuleLoaded("TraceInsights"))
	{
		IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		InsightsModule.UnregisterComponent(InsightsComponent);
	}
	InsightsComponent.Reset();

	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
	IModularFeatures::Get().UnregisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
}

FNiagaraInsightsModule& FNiagaraInsightsModule::Get()
{
	return FModuleManager::LoadModuleChecked<FNiagaraInsightsModule>("NiagaraInsights");
}

} //namespace UE::NiagaraInsights

IMPLEMENT_MODULE(UE::NiagaraInsights::FNiagaraInsightsModule, NiagaraInsights);
