// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "NiagaraTraceModule.h"
#include "NiagaraTimingViewExtender.h"

namespace UE::NiagaraInsights
{

class FNiagaraInsightsComponent;

class FNiagaraInsightsModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	static FNiagaraInsightsModule& Get();

	FNiagaraTimingViewExtender& GetTimingViewExtender() { return TimingViewExtender; }

private:
	void TryRegisterInsightsComponent();
	void OnModulesChanged(FName ModuleName, EModuleChangeReason Reason);

	FNiagaraTraceModule				TraceModule;
	FNiagaraTimingViewExtender		TimingViewExtender;
	TSharedPtr<FNiagaraInsightsComponent> InsightsComponent;
	FDelegateHandle ModuleChangedHandle;
};

} //namespace UE::NiagaraInsights
