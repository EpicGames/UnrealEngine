// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraInstanceLifecycleTrack.h"

#include "NiagaraInsightsCommon.h"
#include "NiagaraProvider.h"
#include "NiagaraTimingViewSession.h"

#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "Insights/ViewModels/GraphTrackEvent.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#define LOCTEXT_NAMESPACE "NiagaraInstanceLifecycleTrack"

namespace UE::NiagaraInsights
{

INSIGHTS_IMPLEMENT_RTTI(FNiagaraInstanceLifecycleTrack)

FNiagaraInstanceLifecycleTrack::FNiagaraInstanceLifecycleTrack(const FNiagaraTimingViewSession& InSharedData)
	: FGraphTrack(LOCTEXT("NiagaraLifecycle", "Niagara Lifecycle").ToString())
	, SharedData(InSharedData)
{
	constexpr double NumEventTypes = 3.0;

	EnabledOptions = //EGraphOptions::ShowDebugInfo |
		EGraphOptions::ShowPoints |
		EGraphOptions::ShowLines |
		EGraphOptions::ShowPolygon |
		EGraphOptions::ShowVerticalAxisGrid |
		EGraphOptions::ShowHeader;

	SetValidLocations(ETimingTrackLocation::Scrollable | ETimingTrackLocation::TopDocked | ETimingTrackLocation::BottomDocked);
	SetHeight(NumEventTypes * 10.0);

	TSharedRef<FGraphSeries> GraphSeries = MakeShared<FGraphSeries>();
	GraphSeries->SetName(TEXT("Events"));
	GraphSeries->SetColor(Common::Color_GTCostTotal);
	GraphSeries->SetBaselineY(GetHeight());
	GraphSeries->SetScaleY(GetHeight() / (NumEventTypes + 1.0));
	GraphSeries->EnableAutoZoom();
	GetSeries().Add(GraphSeries);
}

void FNiagaraInstanceLifecycleTrack::Update(const ITimingTrackUpdateContext& Context)
{
	Super::Update(Context);

	const FNiagaraProvider* Provider = SharedData.GetAnalysisSession().ReadProvider<FNiagaraProvider>(FNiagaraProvider::GetProviderName());
	if (Provider == nullptr)
	{
		return;
	}

	const FTimingTrackViewport& Viewport = Context.GetViewport();

	const TSharedPtr<FGraphSeries> GraphSeries = GetSeries()[0];
	if (GraphSeries->IsVisible())
	{
		FGraphTrackBuilder Builder(*this, *GraphSeries, Viewport);
		Provider->EnumerateComponentEvent(
			Viewport.GetStartTime(),
			Viewport.GetEndTime(),
			[&Builder, &GraphSeries](const FLifetimeEvent& LifetimeEvent)
			{
				const double Offset = double(LifetimeEvent.Payload.GetIndex()) + 0.5;
				Builder.AddEvent(LifetimeEvent.Time, 0.0, Offset, false);
			}
		);
	}
}

void FNiagaraInstanceLifecycleTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	if (!InTooltipEvent.CheckTrack(this) || !InTooltipEvent.Is<FGraphTrackEvent>())
	{
		return;
	}

	const FGraphTrackEvent& GraphEvent = InTooltipEvent.As<FGraphTrackEvent>();
	const TSharedRef<const FGraphSeries> HoveredSeries = GraphEvent.GetSeries();

	const FNiagaraProvider* Provider = SharedData.GetAnalysisSession().ReadProvider<FNiagaraProvider>(FNiagaraProvider::GetProviderName());
	if (Provider == nullptr)
	{
		return;
	}

	InOutTooltip.ResetContent();
	InOutTooltip.AddTitle(HoveredSeries->GetName().ToString(), HoveredSeries->GetColor());

	const double EventTime = GraphEvent.GetStartTime();
	constexpr double TimeTolerance = 1e-9;

	Provider->EnumerateComponentEvent(
		EventTime - TimeTolerance,
		EventTime + TimeTolerance,
		[&InOutTooltip](const FLifetimeEvent& LifetimeEvent)
		{
			switch (LifetimeEvent.Payload.GetIndex())
			{
				case 0:
				{
					const FLifetimeEvent::FActivate& Payload = LifetimeEvent.Payload.Get<FLifetimeEvent::FActivate>();
					InOutTooltip.AddNameValueTextLine(TEXT("Activate"), TEXT(""));
					InOutTooltip.AddNameValueTextLine(TEXT("Component"), LifetimeEvent.ComponentName);
					InOutTooltip.AddNameValueTextLine(TEXT("System"), LifetimeEvent.SystemName);
					InOutTooltip.AddNameValueTextLine(TEXT("bReset"), LexToString(Payload.bReset));
					InOutTooltip.AddNameValueTextLine(TEXT("bIsScalabilityCull"), LexToString(Payload.bIsScalabilityCull));
					InOutTooltip.AddNameValueTextLine(TEXT("bAwaitingActivationDueToNotReady"), LexToString(Payload.bAwaitingActivationDueToNotReady));
					break;
				}

				case 1:
				{
					const FLifetimeEvent::FDeactivate& Payload = LifetimeEvent.Payload.Get<FLifetimeEvent::FDeactivate>();
					InOutTooltip.AddNameValueTextLine(TEXT("DeActivate"), TEXT(""));
					InOutTooltip.AddNameValueTextLine(TEXT("Component"), LifetimeEvent.ComponentName);
					InOutTooltip.AddNameValueTextLine(TEXT("System"), LifetimeEvent.SystemName);
					InOutTooltip.AddNameValueTextLine(TEXT("bImmediate"), LexToString(Payload.bImmediate));
					InOutTooltip.AddNameValueTextLine(TEXT("bIsScalabilityCull"), LexToString(Payload.bIsScalabilityCull));
					break;
				}

				case 2:
				{
					const FLifetimeEvent::FComplete& Payload = LifetimeEvent.Payload.Get<FLifetimeEvent::FComplete>();
					InOutTooltip.AddNameValueTextLine(TEXT("Complete"), TEXT(""));
					InOutTooltip.AddNameValueTextLine(TEXT("Component"), LifetimeEvent.ComponentName);
					InOutTooltip.AddNameValueTextLine(TEXT("System"), LifetimeEvent.SystemName);
					InOutTooltip.AddNameValueTextLine(TEXT("bExternalCompletion"), LexToString(Payload.bExternalCompletion));
					break;
				}
			}
		}
	);

	InOutTooltip.UpdateLayout();
}

} //namespace UE::NiagaraInsights

#undef LOCTEXT_NAMESPACE
