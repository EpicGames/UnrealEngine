// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraPerformanceGraphTrack.h"

#include "NiagaraInsightsCommon.h"
#include "NiagaraProvider.h"
#include "NiagaraTimingViewSession.h"

#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "Insights/ViewModels/GraphTrackEvent.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#define LOCTEXT_NAMESPACE "NiagaraPerformanceGraphTrack"

namespace UE::NiagaraInsights
{

INSIGHTS_IMPLEMENT_RTTI(FNiagaraPerformanceGraphTrack)

FNiagaraPerformanceGraphTrack::FNiagaraPerformanceGraphTrack(const FNiagaraTimingViewSession& InSharedData)
	: FGraphTrack(LOCTEXT("NiagaraPerformance", "Niagara Performance").ToString())
	, SharedData(InSharedData)
{
	EnabledOptions = //EGraphOptions::ShowDebugInfo |
		EGraphOptions::ShowPoints |
		EGraphOptions::ShowLines |
		EGraphOptions::ShowPolygon |
		EGraphOptions::ShowVerticalAxisGrid |
		EGraphOptions::ShowHeader;

	SetValidLocations(ETimingTrackLocation::Scrollable | ETimingTrackLocation::TopDocked | ETimingTrackLocation::BottomDocked);
	SetHeight(200.0f);	// Fixed height for graph

	//-TOOD: Crate series for metrics
	{
		GameTimeSeries = MakeShared<FGraphSeries>();
		GameTimeSeries->SetName(TEXT("Game Time"));
		GameTimeSeries->SetColor(Common::Color_GTCostTotal);
		GameTimeSeries->EnableAutoZoom();
		GetSeries().Add(GameTimeSeries);
	}
	{
		RenderTimeSeries = MakeShared<FGraphSeries>();
		RenderTimeSeries->SetName(TEXT("Render Time"));
		RenderTimeSeries->SetColor(Common::Color_RTCostTotal);
		RenderTimeSeries->EnableAutoZoom();
		GetSeries().Add(RenderTimeSeries);
	}
	{
		GpuTimeSeries = MakeShared<FGraphSeries>();
		GpuTimeSeries->SetName(TEXT("GPU Time"));
		GpuTimeSeries->SetColor(Common::Color_GpuCostTotal);
		GpuTimeSeries->EnableAutoZoom();
		GetSeries().Add(GpuTimeSeries);
	}
	{
		MemoryUsageSeries = MakeShared<FGraphSeries>();
		MemoryUsageSeries->SetName(TEXT("Memory Usage"));
		MemoryUsageSeries->SetColor(Common::Color_MemoryCostTotal);
		MemoryUsageSeries->EnableAutoZoom();
		GetSeries().Add(MemoryUsageSeries);
	}
}

void FNiagaraPerformanceGraphTrack::Update(const ITimingTrackUpdateContext& Context)
{
	Super::Update(Context);

	const FNiagaraProvider* Provider = SharedData.GetAnalysisSession().ReadProvider<FNiagaraProvider>(FNiagaraProvider::GetProviderName());
	if (Provider == nullptr)
	{
		return;
	}

	const FTimingTrackViewport& Viewport = Context.GetViewport();

	// Update height based on visibility
	float SeriesHeight = GetHeight();
	{
		int NumVisible = 0;
		for (TSharedPtr<FGraphSeries>& Series : GetSeries())
		{
			NumVisible += Series->IsVisible() ? 1 : 0;
		}

		if (NumVisible == 0)
		{
			return;
		}

		SeriesHeight = SeriesHeight / double(NumVisible);
		float CurrentSeriesHeight = SeriesHeight;
		for (TSharedPtr<FGraphSeries>& Series : GetSeries())
		{
			if (Series->IsVisible())
			{
				Series->SetBaselineY(CurrentSeriesHeight);
				CurrentSeriesHeight += SeriesHeight;
			}
		}

		// Add on a few pixel so we don't calculate the top point on the line of the next series
		SeriesHeight -= 5.0;
	}

	// Update game time series
	if (GameTimeSeries->IsVisible())
	{
		double MaxValue = 0.0;
		FGraphTrackBuilder Builder(*this, *GameTimeSeries, Viewport);
		Provider->EnumeratePerformance_GT(
			Viewport.GetStartTime(),
			Viewport.GetEndTime(),
			true,
			[&Builder, &MaxValue](const FSystemPerformanceFrame_GT& Frame)
			{
				const FSystemPerformanceFrame_GT::FStats& A = Frame.AccumulatedStats;
				const double Value = (A.TickGameThreadSeconds + A.TickConcurrentSeconds + A.FinalizeSeconds + A.EndOfFrameSeconds + A.ActivationSeconds + A.WaitSeconds) * 1000.0;
				MaxValue = FMath::Max(MaxValue, Value);
				Builder.AddEvent(Frame.Time, 0.0, Value);
			}
		);
		GameTimeSeries->SetScaleY(MaxValue > 0.0 ? SeriesHeight / MaxValue : 1.0);
	}

	// Update render time series
	if (RenderTimeSeries->IsVisible())
	{
		double MaxValue = 0.0;
		FGraphTrackBuilder Builder(*this, *RenderTimeSeries, Viewport);
		Provider->EnumeratePerformance_RT(
			Viewport.GetStartTime(),
			Viewport.GetEndTime(),
			true,
			[&Builder, &MaxValue](const FSystemPerformanceFrame_RT& Frame)
			{
				const double Value = (Frame.AccumulatedStats.RenderUpdateSeconds + Frame.AccumulatedStats.GetDynamicMeshElementsSeconds) * 1000.0;
				MaxValue = FMath::Max(MaxValue, Value);
				Builder.AddEvent(Frame.Time, 0.0, Value);
			}
		);
		RenderTimeSeries->SetScaleY(MaxValue > 0.0 ? SeriesHeight / MaxValue : 1.0);
	}

	// Update gpu time series
	if (GpuTimeSeries->IsVisible())
	{
		double MaxValue = 0.0;
		FGraphTrackBuilder Builder(*this, *GpuTimeSeries, Viewport);
		Provider->EnumeratePerformance_RT(
			Viewport.GetStartTime(),
			Viewport.GetEndTime(),
			true,
			[&Builder, &MaxValue](const FSystemPerformanceFrame_RT& Frame)
			{
				const double Value = Frame.AccumulatedStats.GpuTotalMicroseconds / 1000.0;
				MaxValue = FMath::Max(MaxValue, Value);
				Builder.AddEvent(Frame.Time, 0.0, Value);
			}
		);
		GpuTimeSeries->SetScaleY(MaxValue > 0.0 ? SeriesHeight / MaxValue : 1.0);
	}

	// Update memory series
	if (MemoryUsageSeries->IsVisible())
	{
		double MaxValue = 0.0;
		FGraphTrackBuilder Builder(*this, *MemoryUsageSeries, Viewport);
		Provider->EnumeratePerformance_GT(
			Viewport.GetStartTime(),
			Viewport.GetEndTime(),
			true,
			[&Builder, &MaxValue](const FSystemPerformanceFrame_GT& Frame)
			{
				const double Value = double(Frame.AccumulatedStats.MemoryBytes) / 1024.0;
				MaxValue = FMath::Max(MaxValue, Value);
				Builder.AddEvent(Frame.Time, 0.0, Value);
			}
		);
		MemoryUsageSeries->SetScaleY(MaxValue > 0.0 ? SeriesHeight / MaxValue : 1.0);
	}
}

void FNiagaraPerformanceGraphTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	if (!InTooltipEvent.CheckTrack(this) || !InTooltipEvent.Is<FGraphTrackEvent>())
	{
		return;
	}

	const FGraphTrackEvent& GraphEvent = InTooltipEvent.As<FGraphTrackEvent>();
	const TSharedRef<const FGraphSeries> HoveredSeries = GraphEvent.GetSeries();

	const FNiagaraProvider* Provider = SharedData.GetAnalysisSession().ReadProvider<FNiagaraProvider>(FNiagaraProvider::GetProviderName());
	if (Provider == nullptr)
	{
		return;
	}

	InOutTooltip.ResetContent();
	InOutTooltip.AddTitle(HoveredSeries->GetName().ToString(), HoveredSeries->GetColor());

	// Use a tight window to find the single frame this event was built from.
	// Frames are ~16ms apart so 1ns tolerance should match at most one.
	const double EventTime = GraphEvent.GetStartTime();
	constexpr double TimeTolerance = 1e-9;

	if (HoveredSeries == GameTimeSeries)
	{
		Provider->EnumeratePerformance_GT(
			EventTime - TimeTolerance,
			EventTime + TimeTolerance,
			false,
			[&InOutTooltip](const FSystemPerformanceFrame_GT& Frame)
			{
				struct FSystemEntry { FStringView Name; double TotalMs; uint64 NumInstances; };

				TArray<FSystemEntry> Entries;

				for (const TPair<FString, FSystemPerformanceFrame_GT::FStats>& Pair : Frame.SystemData)
				{
					const FSystemPerformanceFrame_GT::FStats& Sys = Pair.Value;
					const double TotalMs = (Sys.TickGameThreadSeconds + Sys.TickConcurrentSeconds +
						Sys.FinalizeSeconds + Sys.EndOfFrameSeconds +
						Sys.ActivationSeconds + Sys.WaitSeconds) * 1000.0;
					Entries.Add({ Pair.Key, TotalMs, Sys.NumInstances });
				}

				const FSystemPerformanceFrame_GT::FStats& A = Frame.AccumulatedStats;
				const double GrandTotalMs = (A.TickGameThreadSeconds + A.TickConcurrentSeconds + A.FinalizeSeconds + A.EndOfFrameSeconds + A.ActivationSeconds + A.WaitSeconds) * 1000.0;

				Entries.Sort([](const FSystemEntry& A, const FSystemEntry& B) { return A.TotalMs > B.TotalMs; });

				InOutTooltip.AddNameValueTextLine(TEXT("Total:"), FString::Printf(TEXT("%.3f ms"), GrandTotalMs));
				InOutTooltip.AddTextLine(TEXT("Per System (sorted by cost):"), FLinearColor::Gray);

				constexpr int32 MaxSystems = 10;
				for (int32 i = 0; i < FMath::Min(Entries.Num(), MaxSystems); ++i)
				{
					const FSystemEntry& Entry = Entries[i];
					const FString Label = FString::Printf(TEXT("  %s (x%llu)"), *FString(Entry.Name), Entry.NumInstances);
					InOutTooltip.AddNameValueTextLine(*Label, FString::Printf(TEXT("%.3f ms"), Entry.TotalMs));
				}

				if (Entries.Num() > MaxSystems)
				{
					InOutTooltip.AddTextLine(FString::Printf(TEXT("  ... and %d more"), Entries.Num() - MaxSystems), FLinearColor::Gray);
				}
			}
		);
	}
	else if (HoveredSeries == RenderTimeSeries)
	{
		Provider->EnumeratePerformance_RT(
			EventTime - TimeTolerance,
			EventTime + TimeTolerance,
			false,
			[&InOutTooltip](const FSystemPerformanceFrame_RT& Frame)
			{
				struct FSystemEntry { FStringView Name; double TotalMs; uint64 NumInstances; };

				TArray<FSystemEntry> Entries;

				for (const TPair<FString, FSystemPerformanceFrame_RT::FStats>& Pair : Frame.SystemData)
				{
					const FSystemPerformanceFrame_RT::FStats& Sys = Pair.Value;
					const double TotalMs = (Sys.RenderUpdateSeconds + Sys.GetDynamicMeshElementsSeconds) * 1000.0;
					Entries.Add({ Pair.Key, TotalMs, Sys.NumInstances });
				}

				const FSystemPerformanceFrame_RT::FStats& A = Frame.AccumulatedStats;
				const double GrandTotalMs = (A.RenderUpdateSeconds + A.GetDynamicMeshElementsSeconds) * 1000.0;

				Entries.Sort([](const FSystemEntry& A, const FSystemEntry& B) { return A.TotalMs > B.TotalMs; });

				InOutTooltip.AddNameValueTextLine(TEXT("Total:"), FString::Printf(TEXT("%.3f ms"), GrandTotalMs));
				InOutTooltip.AddTextLine(TEXT("Per System (sorted by cost):"), FLinearColor::Gray);

				constexpr int32 MaxSystems = 10;
				for (int32 i = 0; i < FMath::Min(Entries.Num(), MaxSystems); ++i)
				{
					const FSystemEntry& Entry = Entries[i];
					const FString Label = FString::Printf(TEXT("  %s (x%llu)"), *FString(Entry.Name), Entry.NumInstances);
					InOutTooltip.AddNameValueTextLine(*Label, FString::Printf(TEXT("%.3f ms"), Entry.TotalMs));
				}

				if (Entries.Num() > MaxSystems)
				{
					InOutTooltip.AddTextLine(FString::Printf(TEXT("  ... and %d more"), Entries.Num() - MaxSystems), FLinearColor::Gray);
				}
			}
		);
	}
	else if (HoveredSeries == GpuTimeSeries)
	{
		Provider->EnumeratePerformance_RT(
			EventTime - TimeTolerance,
			EventTime + TimeTolerance,
			false,
			[&InOutTooltip](const FSystemPerformanceFrame_RT& Frame)
			{
				struct FSystemEntry { FStringView Name; double TotalMs; uint64 GpuNumInstances; };

				TArray<FSystemEntry> Entries;

				for (const TPair<FString, FSystemPerformanceFrame_RT::FStats>& Pair : Frame.SystemData)
				{
					const FSystemPerformanceFrame_RT::FStats& Sys = Pair.Value;
					if (Sys.GpuNumInstances == 0)
					{
						continue;
					}
					Entries.Add({ Pair.Key, Sys.GpuTotalMicroseconds / 1000.0, Sys.GpuNumInstances });
				}

				const double GrandTotalMs = Frame.AccumulatedStats.GpuTotalMicroseconds / 1000.0;

				Entries.Sort([](const FSystemEntry& A, const FSystemEntry& B) { return A.TotalMs > B.TotalMs; });

				InOutTooltip.AddNameValueTextLine(TEXT("Total:"), FString::Printf(TEXT("%.3f ms"), GrandTotalMs));
				InOutTooltip.AddTextLine(TEXT("Per System (sorted by cost):"), FLinearColor::Gray);

				constexpr int32 MaxSystems = 10;
				for (int32 i = 0; i < FMath::Min(Entries.Num(), MaxSystems); ++i)
				{
					const FSystemEntry& Entry = Entries[i];
					const FString Label = FString::Printf(TEXT("  %s (x%llu)"), *FString(Entry.Name), Entry.GpuNumInstances);
					InOutTooltip.AddNameValueTextLine(*Label, FString::Printf(TEXT("%.3f ms"), Entry.TotalMs));
				}

				if (Entries.Num() > MaxSystems)
				{
					InOutTooltip.AddTextLine(FString::Printf(TEXT("  ... and %d more"), Entries.Num() - MaxSystems), FLinearColor::Gray);
				}
			}
		);
	}
	else if (HoveredSeries == MemoryUsageSeries)
	{
		Provider->EnumeratePerformance_GT(
			EventTime - TimeTolerance,
			EventTime + TimeTolerance,
			false,
			[&InOutTooltip](const FSystemPerformanceFrame_GT& Frame)
			{
				struct FSystemEntry { FStringView Name; uint64 MemoryTotalBytes; uint64 NumInstances; };

				TArray<FSystemEntry> Entries;

				for (const TPair<FString, FSystemPerformanceFrame_GT::FStats>& Pair : Frame.SystemData)
				{
					const FSystemPerformanceFrame_GT::FStats& Sys = Pair.Value;
					Entries.Add({ Pair.Key, Sys.MemoryBytes, Sys.NumInstances });
				}

				const uint64 MemoryTotalBytes = Frame.AccumulatedStats.MemoryBytes;

				Entries.Sort([](const FSystemEntry& A, const FSystemEntry& B) { return A.MemoryTotalBytes > B.MemoryTotalBytes; });

				InOutTooltip.AddNameValueTextLine(TEXT("Total:"), FString::Printf(TEXT("%.3f KB"), double(MemoryTotalBytes) / 1024.0));
				InOutTooltip.AddTextLine(TEXT("Per System (sorted by cost):"), FLinearColor::Gray);

				constexpr int32 MaxSystems = 10;
				for (int32 i = 0; i < FMath::Min(Entries.Num(), MaxSystems); ++i)
				{
					const FSystemEntry& Entry = Entries[i];
					const FString Label = FString::Printf(TEXT("  %s (x%llu)"), *FString(Entry.Name), Entry.NumInstances);
					InOutTooltip.AddNameValueTextLine(*Label, FString::Printf(TEXT("%.3f KB"), double(Entry.MemoryTotalBytes) / 1024.0));
				}

				if (Entries.Num() > MaxSystems)
				{
					InOutTooltip.AddTextLine(FString::Printf(TEXT("  ... and %d more"), Entries.Num() - MaxSystems), FLinearColor::Gray);
				}
			}
		);
	}

	InOutTooltip.UpdateLayout();
}

} //namespace UE::NiagaraInsights

#undef LOCTEXT_NAMESPACE
