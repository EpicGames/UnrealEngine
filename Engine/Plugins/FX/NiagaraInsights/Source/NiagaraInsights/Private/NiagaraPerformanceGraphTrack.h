// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ViewModels/GraphTrack.h"

class FGraphSeries;

namespace UE::NiagaraInsights
{

class FNiagaraTimingViewSession;

class FNiagaraPerformanceGraphTrack : public FGraphTrack
{
	using Super = FGraphTrack;

	INSIGHTS_DECLARE_RTTI(FNiagaraPerformanceGraphTrack, FGraphTrack)

public:
	explicit FNiagaraPerformanceGraphTrack(const FNiagaraTimingViewSession& InSharedData);

	// BEGIN: FGraphTrack interface
	virtual void Update(const ITimingTrackUpdateContext& Context) override;
	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;
	// END: FGraphTrack interface

private:
	const FNiagaraTimingViewSession&	SharedData;
	TSharedPtr<FGraphSeries>			GameTimeSeries;
	TSharedPtr<FGraphSeries>			RenderTimeSeries;
	TSharedPtr<FGraphSeries>			GpuTimeSeries;
	TSharedPtr<FGraphSeries>			MemoryUsageSeries;
};

} //namespace UE::NiagaraInsights
