// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraProvider.h"

#include "Algo/BinarySearch.h"

namespace UE::NiagaraInsights
{
namespace Private
{
	thread_local TraceServices::FProviderLock::FThreadLocalState GProviderLockState;
	static FName ProviderName("NiagaraProvider");

	void ParseComponentName(FLifetimeEvent& Event, FStringView ComponentName)
	{
		const int32 Length = ComponentName.Len();
		int32 FoundIndex = INDEX_NONE;
		if (ComponentName.FindChar(',', FoundIndex) == false)
		{
			FoundIndex = Length;
		}

		Event.ComponentName = ComponentName.Mid(0, FoundIndex);
		Event.SystemName = ComponentName.Mid(FoundIndex + 1, FMath::Max(0, Length - FoundIndex - 1));
	}
}

FNiagaraProvider::FNiagaraProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
{
}

void FNiagaraProvider::BeginRead() const
{
	ProviderLock.BeginRead(Private::GProviderLockState);
}

void FNiagaraProvider::EndRead() const
{
	ProviderLock.EndRead(Private::GProviderLockState);
}

void FNiagaraProvider::ReadAccessCheck() const
{
	ProviderLock.ReadAccessCheck(Private::GProviderLockState);
}

void FNiagaraProvider::BeginEdit() const 
{
	ProviderLock.BeginWrite(Private::GProviderLockState);
}

void FNiagaraProvider::EndEdit() const
{
	ProviderLock.EndWrite(Private::GProviderLockState);
}

void FNiagaraProvider::EditAccessCheck() const
{
	ProviderLock.WriteAccessCheck(Private::GProviderLockState);
}

FName FNiagaraProvider::GetProviderName()
{
	return Private::ProviderName;
}

bool FNiagaraProvider::HasAnyData() const
{
	TraceServices::FProviderReadScopeLock ReadLock(*this);
	return PerformanceFrames_GT.Num() > 0
		|| PerformanceFrames_RT.Num() > 0
		|| LifetimeEvents.Num() > 0
		|| DataChannelEvents.Num() > 0;
}

void FNiagaraProvider::AddSystemPerformance_GT(double EventTime, FStringView SystemName, uint64 NumInstances, double TickGameThreadSeconds, double TickConcurrentSeconds, double FinalizeSeconds, double EndOfFrameSeconds, double ActivationSeconds, double WaitSeconds, uint64 MemoryBytes)
{
	TraceServices::FProviderEditScopeLock WriteLock(*this);

	//int32 iFrame = Algo::BinarySearchBy(PerformanceFrames, EventTime, &FPerformanceFrame::Time, FMath::IsNearlyEqual);
	if (PerformanceFrames_GT.Num() == 0 || !FMath::IsNearlyEqual(PerformanceFrames_GT.Last().Time, EventTime))
	{
		PerformanceFrames_GT.AddDefaulted_GetRef().Time = EventTime;
	}

	FSystemPerformanceFrame_GT& Frame_GT = PerformanceFrames_GT.Last();
	FSystemPerformanceFrame_GT::FStats& Stats_GT = Frame_GT.SystemData.Emplace_GetRef(FString(SystemName), FSystemPerformanceFrame_GT::FStats{}).Value;
	Stats_GT.NumInstances = NumInstances;
	Stats_GT.TickGameThreadSeconds = TickGameThreadSeconds;
	Stats_GT.TickConcurrentSeconds = TickConcurrentSeconds;
	Stats_GT.FinalizeSeconds = FinalizeSeconds;
	Stats_GT.EndOfFrameSeconds = EndOfFrameSeconds;
	Stats_GT.ActivationSeconds = ActivationSeconds;
	Stats_GT.WaitSeconds = WaitSeconds;
	Stats_GT.MemoryBytes = MemoryBytes;

	Frame_GT.AccumulatedStats.NumInstances += NumInstances;
	Frame_GT.AccumulatedStats.TickGameThreadSeconds += TickGameThreadSeconds;
	Frame_GT.AccumulatedStats.TickConcurrentSeconds += TickConcurrentSeconds;
	Frame_GT.AccumulatedStats.FinalizeSeconds += FinalizeSeconds;
	Frame_GT.AccumulatedStats.EndOfFrameSeconds += EndOfFrameSeconds;
	Frame_GT.AccumulatedStats.ActivationSeconds += ActivationSeconds;
	Frame_GT.AccumulatedStats.WaitSeconds += WaitSeconds;
	Frame_GT.AccumulatedStats.MemoryBytes += MemoryBytes;
}

void FNiagaraProvider::AddSystemPerformance_RT(double EventTime, FStringView SystemName, uint64 NumInstances, double RenderUpdateSeconds, double GetDynamicMeshElementsSeconds, uint64 GpuNumInstances, uint64 GpuTotalMicroseconds)
{
	TraceServices::FProviderEditScopeLock WriteLock(*this);

	//int32 iFrame = Algo::BinarySearchBy(PerformanceFrames, EventTime, &FPerformanceFrame::Time, FMath::IsNearlyEqual);
	if (PerformanceFrames_RT.Num() == 0 || !FMath::IsNearlyEqual(PerformanceFrames_RT.Last().Time, EventTime))
	{
		PerformanceFrames_RT.AddDefaulted_GetRef().Time = EventTime;
	}

	FSystemPerformanceFrame_RT& Frame_RT = PerformanceFrames_RT.Last();
	FSystemPerformanceFrame_RT::FStats& Stats_RT = Frame_RT.SystemData.Emplace_GetRef(FString(SystemName), FSystemPerformanceFrame_RT::FStats{}).Value;
	Stats_RT.NumInstances = NumInstances;
	Stats_RT.RenderUpdateSeconds = RenderUpdateSeconds;
	Stats_RT.GetDynamicMeshElementsSeconds = GetDynamicMeshElementsSeconds;
	Stats_RT.GpuNumInstances = GpuNumInstances;
	Stats_RT.GpuTotalMicroseconds = GpuTotalMicroseconds;

	Frame_RT.AccumulatedStats.NumInstances += NumInstances;
	Frame_RT.AccumulatedStats.RenderUpdateSeconds += RenderUpdateSeconds;
	Frame_RT.AccumulatedStats.GetDynamicMeshElementsSeconds += GetDynamicMeshElementsSeconds;
	Frame_RT.AccumulatedStats.GpuNumInstances += GpuNumInstances;
	Frame_RT.AccumulatedStats.GpuTotalMicroseconds += GpuTotalMicroseconds;
}

void FNiagaraProvider::EnumeratePerformance_GT(double StartTime, double EndTime, bool bIncludeOverlapping, TFunction<void(const FSystemPerformanceFrame_GT&)> Evaluate) const
{
	TraceServices::FProviderReadScopeLock ReadLock(*this);

	const int32 Num = PerformanceFrames_GT.Num();
	if (Num == 0)
	{
		return;
	}

	int32 StartIndex = Algo::LowerBoundBy(PerformanceFrames_GT, StartTime, &FSystemPerformanceFrame_GT::Time);
	int32 EndIndex = Algo::UpperBoundBy(PerformanceFrames_GT, EndTime, &FSystemPerformanceFrame_GT::Time);

	if (bIncludeOverlapping)
	{
		StartIndex = FMath::Max(StartIndex - 1, 0);
		EndIndex = FMath::Min(EndIndex + 1, Num);
	}

	for (int32 i = StartIndex; i < EndIndex; ++i)
	{
		Evaluate(PerformanceFrames_GT[i]);
	}
}

void FNiagaraProvider::EnumeratePerformance_RT(double StartTime, double EndTime, bool bIncludeOverlapping, TFunction<void(const FSystemPerformanceFrame_RT&)> Evaluate) const
{
	TraceServices::FProviderReadScopeLock ReadLock(*this);

	const int32 Num = PerformanceFrames_RT.Num();
	if (Num == 0)
	{
		return;
	}

	int32 StartIndex = Algo::LowerBoundBy(PerformanceFrames_RT, StartTime, &FSystemPerformanceFrame_RT::Time);
	int32 EndIndex = Algo::UpperBoundBy(PerformanceFrames_RT, EndTime, &FSystemPerformanceFrame_RT::Time);

	if (bIncludeOverlapping)
	{
		StartIndex = FMath::Max(StartIndex - 1, 0);
		EndIndex = FMath::Min(EndIndex + 1, Num);
	}

	for (int32 i = StartIndex; i < EndIndex; ++i)
	{
		Evaluate(PerformanceFrames_RT[i]);
	}
}

void FNiagaraProvider::AddComponentActivate(double EventTime, FStringView ComponentName, bool bReset, bool bIsScalabilityCull, bool bAwaitingActivationDueToNotReady)
{
	TraceServices::FProviderEditScopeLock WriteLock(*this);

	FLifetimeEvent& Event = LifetimeEvents.AddDefaulted_GetRef();
	Event.Time = EventTime;
	Private::ParseComponentName(Event, ComponentName);
	Event.Payload.Emplace<FLifetimeEvent::FActivate>(bReset, bIsScalabilityCull, bAwaitingActivationDueToNotReady);
}

void FNiagaraProvider::AddComponentDeactivate(double EventTime, FStringView ComponentName, bool bImmediate, bool bIsScalabilityCull)
{
	TraceServices::FProviderEditScopeLock WriteLock(*this);

	FLifetimeEvent& Event = LifetimeEvents.AddDefaulted_GetRef();
	Event.Time = EventTime;
	Private::ParseComponentName(Event, ComponentName);
	Event.Payload.Emplace<FLifetimeEvent::FDeactivate>(bImmediate, bIsScalabilityCull);
}

void FNiagaraProvider::AddComponentComplete(double EventTime, FStringView ComponentName, bool bExternalCompletion)
{
	TraceServices::FProviderEditScopeLock WriteLock(*this);

	FLifetimeEvent& Event = LifetimeEvents.AddDefaulted_GetRef();
	Event.Time = EventTime;
	Private::ParseComponentName(Event, ComponentName);
	Event.Payload.Emplace<FLifetimeEvent::FComplete>(bExternalCompletion);
}

void FNiagaraProvider::EnumerateComponentEvent(double StartTime, double EndTime, TFunction<void(const FLifetimeEvent&)> Evaluate) const
{
	TraceServices::FProviderReadScopeLock ReadLock(*this);

	const int32 StartIndex = Algo::LowerBoundBy(LifetimeEvents, StartTime, &FLifetimeEvent::Time);
	const int32 Num = LifetimeEvents.Num();
	for (int32 i = StartIndex; i < Num; ++i)
	{
		const FLifetimeEvent& Event = LifetimeEvents[i];
		if (Event.Time > EndTime)
		{
			break;
		}
		Evaluate(Event);
	}
}

void FNiagaraProvider::AddDataChannelPublish(double EventTime, FStringView SourceName, bool bGpuRequest, bool bVisibleToGame, bool bVisibleToCPUSims, bool bVisibleToGPUSims, uint32 NumInstances, uint32 NumInstanceAllocated)
{
	TraceServices::FProviderEditScopeLock WriteLock(*this);

	FDataChannelEvent& Event = DataChannelEvents.AddDefaulted_GetRef();
	Event.Time = EventTime;
	Event.Payload.Emplace<FDataChannelEvent::FPublish>(FString(SourceName), bGpuRequest, bVisibleToGame, bVisibleToCPUSims, bVisibleToGPUSims, NumInstances, NumInstanceAllocated);
}

void FNiagaraProvider::AddDataChannelWrite(double EventTime, FStringView DataChannelName, FStringView SourceName, int32 NumInstances, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU)
{
	TraceServices::FProviderEditScopeLock WriteLock(*this);

	FDataChannelEvent& Event = DataChannelEvents.AddDefaulted_GetRef();
	Event.Time = EventTime;
	Event.Payload.Emplace<FDataChannelEvent::FWrite>(FString(DataChannelName), FString(SourceName), NumInstances, bVisibleToGame, bVisibleToCPU, bVisibleToGPU);
}

void FNiagaraProvider::EnumerateDataChannelEvent(double StartTime, double EndTime, TFunction<void(const FDataChannelEvent&)> Evaluate) const
{
	TraceServices::FProviderReadScopeLock ReadLock(*this);

	const int32 StartIndex = Algo::LowerBoundBy(DataChannelEvents, StartTime, &FDataChannelEvent::Time);
	const int32 Num = DataChannelEvents.Num();
	for (int32 i = StartIndex; i < Num; ++i)
	{
		const FDataChannelEvent& Event = DataChannelEvents[i];
		if (Event.Time > EndTime)
		{
			break;
		}
		Evaluate(Event);
	}
}

} //namespace UE::NiagaraInsights
