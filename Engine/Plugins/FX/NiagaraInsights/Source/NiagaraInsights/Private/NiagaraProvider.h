// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/ProviderLock.h"
#include "InsightsCore/Common/SimpleRtti.h"
#include "Model/IntervalTimeline.h"
#include "Trace/Analyzer.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace TraceServices { class IAnalysisSession; }

namespace UE::NiagaraInsights
{
struct FSystemPerformanceFrame_GT
{
	struct FStats
	{
		uint64 NumInstances = 0;
		double TickGameThreadSeconds = 0.0;
		double TickConcurrentSeconds = 0.0;
		double FinalizeSeconds = 0.0;
		double EndOfFrameSeconds = 0.0;
		double ActivationSeconds = 0.0;
		double WaitSeconds = 0.0;
		uint64 MemoryBytes = 0;
	};

	double Time = 0.0;
	TArray<TPair<FString, FStats>> SystemData;
	FStats AccumulatedStats;
};

struct FSystemPerformanceFrame_RT
{
	struct FStats
	{
		uint64 NumInstances = 0;
		double RenderUpdateSeconds = 0.0;
		double GetDynamicMeshElementsSeconds = 0.0;

		uint64 GpuNumInstances = 0;
		uint64 GpuTotalMicroseconds = 0;
	};

	double Time = 0.0;
	TArray<TPair<FString, FStats>> SystemData;
	FStats AccumulatedStats;
};

struct FLifetimeEvent
{
	struct FActivate
	{
		bool bReset = false;
		bool bIsScalabilityCull = false;
		bool bAwaitingActivationDueToNotReady = false;
	};
	struct FDeactivate
	{
		bool bImmediate = false;
		bool bIsScalabilityCull = false;
	};
	struct FComplete
	{
		bool bExternalCompletion = false;
	};

	double Time = 0.0;
	FString ComponentName;
	FString SystemName;
	TVariant<FActivate, FDeactivate, FComplete> Payload;
};

struct FDataChannelEvent
{
	struct FPublish
	{
		FString SourceName;
		bool bGpuRequest = false;
		bool bVisibleToGame = false;
		bool bVisibleToCPUSims = false;
		bool bVisibleToGPUSims = false;
		uint32 NumInstances = 0;
		uint32 NumInstanceAllocated = 0;
	};

	struct FWrite
	{
		FString DataChannelName;
		FString SourceName;
		int32	NumInstances = 0;
		bool	bVisibleToGame = false;
		bool	bVisibleToCPU = false;
		bool	bVisibleToGPU = false;
	};

	double Time = 0.0;
	TVariant<FPublish, FWrite> Payload;
};

class FNiagaraProvider : public TraceServices::IProvider, public TraceServices::IEditableProvider
{
public:
	explicit FNiagaraProvider(TraceServices::IAnalysisSession& InSession);

	// Begin: TraceServices::IProvider Impl
	virtual void BeginRead() const override;
	virtual void EndRead() const override;
	virtual void ReadAccessCheck() const override;
	// End: TraceServices::IProvider Impl

	// Begin: TraceServices::IEditableProvider Impl
	virtual void BeginEdit() const override;
	virtual void EndEdit() const override;
	virtual void EditAccessCheck() const override;
	// End: TraceServices::IEditableProvider Impl

	static FName GetProviderName();

	// Returns true if any trace events have been received
	bool HasAnyData() const;

	// Add system performance data for the game thread
	void AddSystemPerformance_GT(double EventTime, FStringView SystemName, uint64 NumInstances, double TickGameThreadSeconds, double TickConcurrentSeconds, double FinalizeSeconds, double EndOfFrameSeconds, double ActivationSeconds, double WaitSeconds, uint64 MemoryBytes);
	// Add system performnace data for the render thread
	void AddSystemPerformance_RT(double EventTime, FStringView SystemName, uint64 NumInstances, double RenderUpdateSeconds, double GetDynamicMeshElementsSeconds, uint64 GpuNumInstances, uint64 GpuTotalMicroseconds);

	// Enumerate a range of game thread frames
	void EnumeratePerformance_GT(double StartTime, double EndTime, bool bIncludeOverlapping, TFunction<void(const FSystemPerformanceFrame_GT&)> Evaluate) const;
	// Enumerate a range of render thread frames
	void EnumeratePerformance_RT(double StartTime, double EndTime, bool bIncludeOverlapping, TFunction<void(const FSystemPerformanceFrame_RT&)> Evaluate) const;

	// Add component activate event
	void AddComponentActivate(double EventTime, FStringView ComponentName, bool bReset, bool bIsScalabilityCull, bool bAwaitingActivationDueToNotReady);
	// Add component deactivate event
	void AddComponentDeactivate(double EventTime, FStringView ComponentName, bool bImmediate, bool bIsScalabilityCull);
	// Add component complete event
	void AddComponentComplete(double EventTime, FStringView ComponentName, bool bExternalCompletion);

	// Enumerate a range of component events
	void EnumerateComponentEvent(double StartTime, double EndTime, TFunction<void(const FLifetimeEvent&)> Evaluate) const;

	// Add data channel publish event
	void AddDataChannelPublish(double EventTime, FStringView SourceName, bool bGpuRequest, bool bVisibleToGame, bool bVisibleToCPUSims, bool bVisibleToGPUSims, uint32 NumInstances, uint32 NumInstanceAllocated);
	// Add data channel write event
	void AddDataChannelWrite(double EventTime, FStringView DataChannelName, FStringView SourceName, int32 NumInstances, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU);

	// Enumerate a range of data channel events
	void EnumerateDataChannelEvent(double StartTime, double EndTime, TFunction<void(const FDataChannelEvent&)> Evaluate) const;

private:
	mutable TraceServices::FProviderLock	ProviderLock;

	TraceServices::IAnalysisSession&		Session;

	TArray<FSystemPerformanceFrame_GT>		PerformanceFrames_GT;
	TArray<FSystemPerformanceFrame_RT>		PerformanceFrames_RT;

	TArray<FLifetimeEvent>					LifetimeEvents;

	TArray<FDataChannelEvent>				DataChannelEvents;
};

} //namespace UE::NiagaraInsights
