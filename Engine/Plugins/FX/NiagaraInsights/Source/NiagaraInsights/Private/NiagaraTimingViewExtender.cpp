// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraTimingViewExtender.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/IUnrealInsightsModule.h"

#define LOCTEXT_NAMESPACE "NiagaraTimingViewExtender"

namespace UE::NiagaraInsights
{

void FNiagaraTimingViewExtender::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if (PerSessionData == nullptr)
	{
		PerSessionData = &PerSessionDataMap.Add(&InSession);
		PerSessionData->SharedData = MakeUnique<FNiagaraTimingViewSession>();
	}

	PerSessionData->SharedData->OnBeginSession(InSession);
	OnSessionBegun.Broadcast(*PerSessionData->SharedData);
}

FNiagaraTimingViewSession* FNiagaraTimingViewExtender::FindFirstActiveSession()
{
	for (auto& KV : PerSessionDataMap)
	{
		if (KV.Value.SharedData.IsValid())
		{
			return KV.Value.SharedData.Get();
		}
	}
	return nullptr;
}

void FNiagaraTimingViewExtender::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData != nullptr)
	{
		OnSessionEnded.Broadcast(*PerSessionData->SharedData);
		PerSessionData->SharedData->OnEndSession(InSession);
	}

	PerSessionDataMap.Remove(&InSession);
}

void FNiagaraTimingViewExtender::Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData != nullptr)
	{
		PerSessionData->SharedData->Tick(InSession, InAnalysisSession);
	}
}

void FNiagaraTimingViewExtender::ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder)
{
	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if (PerSessionData != nullptr)
	{
		PerSessionData->SharedData->ExtendFilterMenu(InMenuBuilder);
	}
}

} //namespace UE::NiagaraInsights

#undef LOCTEXT_NAMESPACE
