// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ITimingViewExtender.h"
#include "NiagaraTimingViewSession.h"

namespace UE::Insights::Timing { class ITimingViewSession; }
namespace TraceServices { class IAnalysisSession; }

class FMenuBuilder;

namespace UE::NiagaraInsights
{

class FNiagaraTimingViewExtender : public UE::Insights::Timing::ITimingViewExtender
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSessionBegun, FNiagaraTimingViewSession& /*Session*/);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSessionEnded, FNiagaraTimingViewSession& /*Session*/);

	//~ Begin UE::Insights::Timing::ITimingViewExtender impl
	virtual void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
	virtual void ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;
	//~ End UE::Insights::Timing::ITimingViewExtender impl

	// Returns the first active Niagara timing-view session, or nullptr if none has started.
	// The stats view uses this to bind to the range-selection delegate.
	FNiagaraTimingViewSession* FindFirstActiveSession();

	// Fired when a new timing-view session begins (after tracks are created).
	// Observers can use this to late-bind to the session's OnRangeChanged delegate.
	FOnSessionBegun OnSessionBegun;

	// Fired when a timing-view session is about to be destroyed.
	// Observers must release any raw pointers to the session before returning.
	FOnSessionEnded OnSessionEnded;

private:
	struct FPerSessionData
	{
		TUniquePtr<FNiagaraTimingViewSession> SharedData;
	};

	TMap<UE::Insights::Timing::ITimingViewSession*, FPerSessionData> PerSessionDataMap;
};

} //namespace UE::NiagaraInsights
