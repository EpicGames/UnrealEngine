// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraTimingViewSession.h"
#include "NiagaraDataChannelTrack.h"
#include "NiagaraInstanceLifecycleTrack.h"
#include "NiagaraPerformanceGraphTrack.h"
#include "NiagaraProvider.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Insights/ITimingViewSession.h"

#define LOCTEXT_NAMESPACE "NiagaraTimingViewSession"

namespace UE::NiagaraInsights
{

void FNiagaraTimingViewSession::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	TimingViewSession = &InTimingViewSession;

	DataChannelTrack.Reset();
	InstanceLifecycleTrack.Reset();
	PerformanceGraphTrack.Reset();

	InTimingViewSession.OnSelectionChanged().AddRaw(this, &FNiagaraTimingViewSession::HandleTimingViewSelectionChanged);
}

void FNiagaraTimingViewSession::OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	InTimingViewSession.OnSelectionChanged().RemoveAll(this);

	DataChannelTrack.Reset();
	InstanceLifecycleTrack.Reset();
	PerformanceGraphTrack.Reset();
	TimingViewSession = nullptr;
	AnalysisSession   = nullptr;
}

void FNiagaraTimingViewSession::HandleTimingViewSelectionChanged(UE::Insights::Timing::ETimeChangedFlags /*Flags*/, double StartTime, double EndTime)
{
	SelectionStartTime = StartTime;
	SelectionEndTime   = EndTime;
	OnRangeChanged.Broadcast(StartTime, EndTime);
}

void FNiagaraTimingViewSession::Tick(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	AnalysisSession = &InAnalysisSession;

	const FNiagaraProvider* NiagaraProvider = InAnalysisSession.ReadProvider<FNiagaraProvider>(FNiagaraProvider::GetProviderName());

	if (NiagaraProvider)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

		if (!InstanceLifecycleTrack.IsValid() && NiagaraProvider->HasAnyData())
		{
			constexpr int32 TrackOrder = MAX_int32 / 2;
		
			PerformanceGraphTrack = MakeShared<FNiagaraPerformanceGraphTrack>(*this);
			PerformanceGraphTrack->SetVisibilityFlag(bTrackVisible);
			PerformanceGraphTrack->SetOrder(TrackOrder);
			InTimingViewSession.AddScrollableTrack(PerformanceGraphTrack);

			InstanceLifecycleTrack = MakeShared<FNiagaraInstanceLifecycleTrack>(*this);
			InstanceLifecycleTrack->SetVisibilityFlag(bTrackVisible);
			InstanceLifecycleTrack->SetOrder(TrackOrder + 1);
			InTimingViewSession.AddScrollableTrack(InstanceLifecycleTrack);

			DataChannelTrack = MakeShared<FNiagaraDataChannelTrack>(*this);
			DataChannelTrack->SetVisibilityFlag(bTrackVisible);
			DataChannelTrack->SetOrder(TrackOrder + 2);
			InTimingViewSession.AddScrollableTrack(DataChannelTrack);

			InTimingViewSession.InvalidateScrollableTracksOrder();
		}
	}
}

void FNiagaraTimingViewSession::ExtendFilterMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("NiagaraTracks", LOCTEXT("NiagaraHeader", "Niagara"));
	{
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("NiagaraTimingTracks", "Niagara Tracks"),
			LOCTEXT("NiagaraTimingTracks_Tooltip", "Show/hide the Niagara track"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FNiagaraTimingViewSession::ToggleNiagaraTrack),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return bTrackVisible; })
				),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	InMenuBuilder.EndSection();
}

void FNiagaraTimingViewSession::ToggleNiagaraTrack()
{
	bTrackVisible = !bTrackVisible;

	if (DataChannelTrack)
	{
		DataChannelTrack->SetVisibilityFlag(bTrackVisible);
	}
	if (InstanceLifecycleTrack)
	{
		InstanceLifecycleTrack->SetVisibilityFlag(bTrackVisible);
	}
	if (PerformanceGraphTrack)
	{
		PerformanceGraphTrack->SetVisibilityFlag(bTrackVisible);
	}
}

} //namespace UE::NiagaraInsights

#undef LOCTEXT_NAMESPACE
