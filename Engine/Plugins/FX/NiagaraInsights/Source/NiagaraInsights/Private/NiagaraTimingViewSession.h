// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

namespace TraceServices { class IAnalysisSession; }
namespace UE::Insights::Timing { class ITimingViewSession; }
namespace UE::Insights::Timing { enum class ETimeChangedFlags : int32; }
class FMenuBuilder;

namespace UE::NiagaraInsights
{

class FNiagaraDataChannelTrack;
class FNiagaraInstanceLifecycleTrack;
class FNiagaraPerformanceGraphTrack;

/** Fired when the user changes the time range selection in the Timing Profiler. */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNiagaraTimingRangeChanged, double /*StartTime*/, double /*EndTime*/);

class FNiagaraTimingViewSession
{
public:
	void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void Tick(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession);
	void ExtendFilterMenu(FMenuBuilder& InMenuBuilder);

	bool IsAnalysisSessionValid() const { return AnalysisSession != nullptr; }
	const TraceServices::IAnalysisSession& GetAnalysisSession() const { check(AnalysisSession); return *AnalysisSession; }

	double GetSelectionStartTime() const { return SelectionStartTime; }
	double GetSelectionEndTime()   const { return SelectionEndTime; }

	/** Broadcast when the timing-view selection range changes. */
	FOnNiagaraTimingRangeChanged OnRangeChanged;

	// Show / hide the track
	void ToggleNiagaraTrack();

private:
	void HandleTimingViewSelectionChanged(UE::Insights::Timing::ETimeChangedFlags Flags, double StartTime, double EndTime);

	const TraceServices::IAnalysisSession*		AnalysisSession = nullptr;
	UE::Insights::Timing::ITimingViewSession*	TimingViewSession = nullptr;

	TSharedPtr<FNiagaraDataChannelTrack>		DataChannelTrack;
	TSharedPtr<FNiagaraInstanceLifecycleTrack>	InstanceLifecycleTrack;
	TSharedPtr<FNiagaraPerformanceGraphTrack>	PerformanceGraphTrack;

	bool   bTrackVisible      = true;
	double SelectionStartTime = 0.0;
	double SelectionEndTime   = 0.0;
};

} //namespace UE::NiagaraInsights
