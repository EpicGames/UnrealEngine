// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraTraceModule.h"
#include "NiagaraProvider.h"
#include "NiagaraAnalyzer.h"

namespace UE::NiagaraInsights
{
namespace Private
{
static FName ModuleName("TraceModule_Niagara");
}

void FNiagaraTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = Private::ModuleName;
	OutModuleInfo.DisplayName = TEXT("Niagara");
}

void FNiagaraTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FNiagaraProvider> Provider = MakeShared<FNiagaraProvider>(InSession);
	InSession.AddProvider(FNiagaraProvider::GetProviderName(), Provider);

	InSession.AddAnalyzer(new FNiagaraAnalyzer(InSession, *Provider));
}

void FNiagaraTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("Niagara"));
}

void FNiagaraTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{

}

} //namespace UE::NiagaraInsights
