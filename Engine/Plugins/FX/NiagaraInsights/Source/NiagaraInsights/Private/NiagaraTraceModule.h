// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/ModuleService.h"

namespace UE::NiagaraInsights
{

class FNiagaraTraceModule : public TraceServices::IModule
{
public:
	//~ Begin TraceServices::IModule interface
	virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
	virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& Session) override;
	virtual void GetLoggers(TArray<const TCHAR *>& OutLoggers) override;
	virtual void GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory) override;
	virtual const TCHAR* GetCommandLineArgument() override { return TEXT("niagaratrace"); }
	//~ End TraceServices::IModule interface
};

} //namespace UE::NiagaraInsights
