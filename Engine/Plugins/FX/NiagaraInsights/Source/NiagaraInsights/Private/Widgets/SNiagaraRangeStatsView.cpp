// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraRangeStatsView.h"

#include "NiagaraInsightsCommon.h"
#include "NiagaraInsightsModule.h"
#include "NiagaraProvider.h"
#include "NiagaraTimingViewExtender.h"
#include "NiagaraTimingViewSession.h"

#include "Insights/IUnrealInsightsModule.h"
#include "Modules/ModuleManager.h"
#include "TraceServices/Model/AnalysisSession.h"

#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "SNiagaraRangeStatsView"

namespace UE::NiagaraInsights
{

// ---- Column IDs ----------------------------------------------------------------

static const FName ColId_Rank(TEXT("Rank"));
static const FName ColId_System(TEXT("System"));
static const FName ColId_Primary(TEXT("Primary"));
static const FName ColId_Secondary(TEXT("Secondary"));

// ---- STableRow for a stats row -------------------------------------------------

class SNiagaraStatsTableRow : public SMultiColumnTableRow<FNiagaraStatsRowPtr>
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStatsTableRow) {}
		SLATE_ARGUMENT(FNiagaraStatsRowPtr, Row)
		SLATE_ARGUMENT(FText, PrimaryColumnLabel)
		SLATE_ARGUMENT(FText, SecondaryColumnLabel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		Row                  = InArgs._Row;
		PrimaryColumnLabel   = InArgs._PrimaryColumnLabel;
		SecondaryColumnLabel = InArgs._SecondaryColumnLabel;
		SMultiColumnTableRow::Construct(SMultiColumnTableRow::FArguments(), InOwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override
	{
		if (!Row.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		const FMargin CellPad(4.f, 2.f);

		if (ColumnId == ColId_Rank)
		{
			return SNew(SBox).Padding(CellPad).HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::AsNumber(Row->Rank))
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				];
		}
		if (ColumnId == ColId_System)
		{
			return SNew(SBox).Padding(CellPad)
				[
					SNew(STextBlock)
					.Text(FText::FromString(Row->SystemName))
					.ToolTipText(FText::FromString(Row->SystemName))
				];
		}
		if (ColumnId == ColId_Primary)
		{
			const FString Formatted = FString::Printf(TEXT("%.3f %s"), Row->PrimaryValue, *Row->PrimaryUnit);
			return SNew(SBox).Padding(CellPad).HAlign(HAlign_Right)
				[
					SNew(STextBlock).Text(FText::FromString(Formatted))
				];
		}
		if (ColumnId == ColId_Secondary)
		{
			const FString Formatted = FString::Printf(TEXT("%.3f %s"), Row->SecondaryValue, *Row->SecondaryUnit);
			return SNew(SBox).Padding(CellPad).HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(FText::FromString(Formatted))
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				];
		}
		return SNullWidget::NullWidget;
	}

private:
	FNiagaraStatsRowPtr Row;
	FText PrimaryColumnLabel;
	FText SecondaryColumnLabel;
};

// ---- SNiagaraRangeStatsView ----------------------------------------------------

void SNiagaraRangeStatsView::Construct(const FArguments& InArgs)
{
	RangeText      = LOCTEXT("NoRange",    "No range selected");
	FrameCountText = LOCTEXT("NoSession",  "No active Niagara session");

	// Seed empty rows so the list views have something to bind against.
	auto MakeEmpty = [](TArray<FNiagaraStatsRowPtr>& Rows)
	{
		for (int32 i = 1; i <= 5; ++i)
		{
			auto Row = MakeShared<FNiagaraStatsRow>();
			Row->Rank = i;
			Row->SystemName = TEXT("—");
			Rows.Add(Row);
		}
	};
	MakeEmpty(Rows_InstanceCount);
	MakeEmpty(Rows_GTCostTotal);
	MakeEmpty(Rows_GTCostPerInst);
	MakeEmpty(Rows_RTCostTotal);
	MakeEmpty(Rows_RTCostPerInst);
	MakeEmpty(Rows_GpuCostTotal);

	ChildSlot
	[
		SNew(SVerticalBox)

		// ---- Range info header ----
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.f, 4.f, 4.f, 2.f)
		[
			MakeRangeHeaderBar()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
		]

		// ---- Six top-5 sections in a scroll box ----
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SScrollBox)

			+ SScrollBox::Slot()
			[
				MakeStatsSection(
					LOCTEXT("Sec_InstanceCount", "Top 5 by Instance Count"),
					Common::Color_InstanceCount,
					LOCTEXT("Col_AvgInstances", "Avg Instances (GT)"),
					LOCTEXT("Col_TotalGTms",    "Total GT (ms)"),
					Rows_InstanceCount)
			]

			+ SScrollBox::Slot()
			.Padding(0.f, 4.f, 0.f, 0.f)
			[
				MakeStatsSection(
					LOCTEXT("Sec_GTCostTotal", "Top 5 by Game Thread Cost"),
					Common::Color_GTCostTotal,
					LOCTEXT("Col_TotalGTms2",   "Total GT (ms)"),
					LOCTEXT("Col_AvgInst2",     "Avg Instances"),
					Rows_GTCostTotal)
			]

			+ SScrollBox::Slot()
			.Padding(0.f, 4.f, 0.f, 0.f)
			[
				MakeStatsSection(
					LOCTEXT("Sec_GTCostPerInst", "Top 5 by GT Cost / Instance"),
					Common::Color_GTCostPerInst,
					LOCTEXT("Col_GTmsPerInst",  "GT ms / Instance"),
					LOCTEXT("Col_TotalGT3",     "Total GT (ms)"),
					Rows_GTCostPerInst)
			]

			+ SScrollBox::Slot()
			.Padding(0.f, 4.f, 0.f, 0.f)
			[
				MakeStatsSection(
					LOCTEXT("Sec_RTCostTotal", "Top 5 by Render Thread Cost"),
					Common::Color_RTCostTotal,
					LOCTEXT("Col_TotalRTms",    "Total RT (ms)"),
					LOCTEXT("Col_AvgInst3",     "Avg Instances"),
					Rows_RTCostTotal)
			]

			+ SScrollBox::Slot()
			.Padding(0.f, 4.f, 0.f, 0.f)
			[
				MakeStatsSection(
					LOCTEXT("Sec_RTCostPerInst", "Top 5 by RT Cost / Instance"),
					Common::Color_RTCostPerInst,
					LOCTEXT("Col_RTmsPerInst",  "RT ms / Instance"),
					LOCTEXT("Col_TotalRT3",     "Total RT (ms)"),
					Rows_RTCostPerInst)
			]

			+ SScrollBox::Slot()
			.Padding(0.f, 4.f, 0.f, 4.f)
			[
				MakeStatsSection(
					LOCTEXT("Sec_GpuCostTotal", "Top 5 by GPU Cost"),
					Common::Color_GpuCostTotal,
					LOCTEXT("Col_TotalGPUms",   "Total GPU (ms)"),
					LOCTEXT("Col_GpuInstances", "GPU Instances"),
					Rows_GpuCostTotal)
			]
		]
	];

	// Bind to the extender if a session is already active.
	TryBindToActiveSession();

	// Also subscribe to future sessions.
	if (FModuleManager::Get().IsModuleLoaded("NiagaraInsights"))
	{
		FNiagaraInsightsModule& Module = FNiagaraInsightsModule::Get();
		Module.GetTimingViewExtender().OnSessionBegun.AddSP(this, &SNiagaraRangeStatsView::OnNiagaraTimingSessionBegun);
		Module.GetTimingViewExtender().OnSessionEnded.AddSP(this, &SNiagaraRangeStatsView::OnNiagaraTimingSessionEnded);
	}
}

SNiagaraRangeStatsView::~SNiagaraRangeStatsView()
{
	UnbindCurrentSession();

	if (FModuleManager::Get().IsModuleLoaded("NiagaraInsights"))
	{
		FNiagaraInsightsModule& Module = FNiagaraInsightsModule::Get();
		Module.GetTimingViewExtender().OnSessionBegun.RemoveAll(this);
		Module.GetTimingViewExtender().OnSessionEnded.RemoveAll(this);
	}
}

// ---- Session binding -----------------------------------------------------------

void SNiagaraRangeStatsView::TryBindToActiveSession()
{
	if (!FModuleManager::Get().IsModuleLoaded("NiagaraInsights"))
	{
		return;
	}

	FNiagaraTimingViewSession* Session =
		FNiagaraInsightsModule::Get().GetTimingViewExtender().FindFirstActiveSession();

	if (Session && Session != BoundSession)
	{
		UnbindCurrentSession();

		BoundSession     = Session;
		RangeChangedHandle = Session->OnRangeChanged.AddSP(
			this, &SNiagaraRangeStatsView::OnTimingRangeChanged);
	}
}

void SNiagaraRangeStatsView::OnNiagaraTimingSessionEnded(FNiagaraTimingViewSession& Session)
{
	if (&Session == BoundSession)
	{
		UnbindCurrentSession();
	}
}

void SNiagaraRangeStatsView::OnNiagaraTimingSessionBegun(FNiagaraTimingViewSession& Session)
{
	if (&Session == BoundSession)
	{
		return;
	}
	UnbindCurrentSession();

	BoundSession       = &Session;
	RangeChangedHandle = Session.OnRangeChanged.AddSP(
		this, &SNiagaraRangeStatsView::OnTimingRangeChanged);
}

void SNiagaraRangeStatsView::UnbindCurrentSession()
{
	if (BoundSession)
	{
		BoundSession->OnRangeChanged.Remove(RangeChangedHandle);
		BoundSession = nullptr;
	}
	RangeChangedHandle.Reset();
}

void SNiagaraRangeStatsView::OnTimingRangeChanged(double StartTime, double EndTime)
{
	RebuildStats(StartTime, EndTime);
}

// ---- Data aggregation ----------------------------------------------------------

void SNiagaraRangeStatsView::RebuildStats(double StartTime, double EndTime)
{
	CurrentStartTime = StartTime;
	CurrentEndTime   = EndTime;

	const double DurationS = EndTime - StartTime;

	// Update header labels.
	if (RangeTextWidget.IsValid())
	{
		RangeTextWidget->SetText(FText::FromString(
			FString::Printf(TEXT("Range: %.4f s  →  %.4f s   (%.4f s)"),
				StartTime, EndTime, DurationS)));
	}

	// Fetch the analysis session.
	if (!FModuleManager::Get().IsModuleLoaded("TraceInsights"))
	{
		return;
	}
	IUnrealInsightsModule& InsightsModule =
		FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	TSharedPtr<const TraceServices::IAnalysisSession> AnalysisSession =
		InsightsModule.GetAnalysisSession();
	if (!AnalysisSession.IsValid())
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope ReadScope(*AnalysisSession);

	const FNiagaraProvider* Provider =
		AnalysisSession->ReadProvider<FNiagaraProvider>(FNiagaraProvider::GetProviderName());
	if (!Provider)
	{
		return;
	}

	// Accumulate per-system data across the range.
	TMap<FString, FAggregated> AggMap;

	int32 GTFrameCount = 0;
	Provider->EnumeratePerformance_GT(
		StartTime,
		EndTime,
		false,
		[&](const FSystemPerformanceFrame_GT& Frame)
		{
			++GTFrameCount;
			for (const TPair<FString, FSystemPerformanceFrame_GT::FStats>& Pair : Frame.SystemData)
			{
				const FSystemPerformanceFrame_GT::FStats& Sys = Pair.Value;
				FAggregated& A = AggMap.FindOrAdd(Pair.Key);
				A.SystemName = Pair.Key;

				A.GTSamples++;
				A.GTTotalInstances += (double)Sys.NumInstances;
				A.GTTotalMs +=
					(Sys.TickGameThreadSeconds +
					 Sys.TickConcurrentSeconds +
					 Sys.FinalizeSeconds +
					 Sys.EndOfFrameSeconds +
					 Sys.ActivationSeconds +
					 Sys.WaitSeconds) * 1000.0;
			}
		});

	Provider->EnumeratePerformance_RT(
		StartTime,
		EndTime,
		false,
		[&](const FSystemPerformanceFrame_RT& Frame)
		{
			for (const TPair<FString, FSystemPerformanceFrame_RT::FStats>& Pair : Frame.SystemData)
			{
				const FSystemPerformanceFrame_RT::FStats& Sys = Pair.Value;
				FAggregated& A = AggMap.FindOrAdd(Pair.Key);
				A.SystemName = Pair.Key;

				A.RTSamples++;
				A.RTTotalInstances += (double)Sys.NumInstances;
				A.RTTotalMs        +=
					(Sys.RenderUpdateSeconds + Sys.GetDynamicMeshElementsSeconds) * 1000.0;
				A.GpuTotalMs  += (double)Sys.GpuTotalMicroseconds / 1000.0;
				A.GpuTotalInst += (double)Sys.GpuNumInstances;
			}
		});

	if (FrameCountWidget.IsValid())
	{
		FrameCountWidget->SetText(FText::FromString(
			FString::Printf(TEXT("%d GT frames  |  %d systems"),
				GTFrameCount, AggMap.Num())));
	}

	// Convert map to array for sorting.
	TArray<FAggregated> AggArray;
	AggArray.Reserve(AggMap.Num());
	for (auto& KV : AggMap)
	{
		AggArray.Add(MoveTemp(KV.Value));
	}

	// Build six top-5 lists.
	Rows_InstanceCount = BuildTop5(AggArray,
		[](const FAggregated& A) { return (float)A.AvgGTInstances(); },  TEXT("inst"),
		[](const FAggregated& A) { return (float)A.GTTotalMs; },         TEXT("ms"));

	Rows_GTCostTotal = BuildTop5(AggArray,
		[](const FAggregated& A) { return (float)A.GTTotalMs; },         TEXT("ms"),
		[](const FAggregated& A) { return (float)A.AvgGTInstances(); },  TEXT("inst"));

	Rows_GTCostPerInst = BuildTop5(AggArray,
		[](const FAggregated& A) { return (float)A.GTCostPerInst(); },   TEXT("ms/inst"),
		[](const FAggregated& A) { return (float)A.GTTotalMs; },         TEXT("ms"));

	Rows_RTCostTotal = BuildTop5(AggArray,
		[](const FAggregated& A) { return (float)A.RTTotalMs; },         TEXT("ms"),
		[](const FAggregated& A) { return (float)A.RTTotalInstances / FMath::Max(1.0, (double)A.RTSamples); }, TEXT("inst"));

	Rows_RTCostPerInst = BuildTop5(AggArray,
		[](const FAggregated& A) { return (float)A.RTCostPerInst(); },   TEXT("ms/inst"),
		[](const FAggregated& A) { return (float)A.RTTotalMs; },         TEXT("ms"));

	Rows_GpuCostTotal = BuildTop5(AggArray,
		[](const FAggregated& A) { return (float)A.GpuTotalMs; },        TEXT("ms"),
		[](const FAggregated& A) { return (float)A.GpuCostPerInst(); },  TEXT("ms/inst"));

	// Refresh all list views.
	auto Refresh = [](TSharedPtr<SListView<FNiagaraStatsRowPtr>>& LV)
	{
		if (LV.IsValid()) { LV->RequestListRefresh(); }
	};
	Refresh(ListView_InstanceCount);
	Refresh(ListView_GTCostTotal);
	Refresh(ListView_GTCostPerInst);
	Refresh(ListView_RTCostTotal);
	Refresh(ListView_RTCostPerInst);
	Refresh(ListView_GpuCostTotal);
}

// ---- Build top-5 helper --------------------------------------------------------

TArray<FNiagaraStatsRowPtr> SNiagaraRangeStatsView::BuildTop5(
	const TArray<FAggregated>& Data,
	TFunctionRef<float(const FAggregated&)> PrimaryGetter,
	const FString& PrimaryUnit,
	TFunctionRef<float(const FAggregated&)> SecondaryGetter,
	const FString& SecondaryUnit)
{
	// Sort descending by primary metric.
	TArray<const FAggregated*> Sorted;
	Sorted.Reserve(Data.Num());
	for (const FAggregated& A : Data)
	{
		Sorted.Add(&A);
	}
	Sorted.Sort([&PrimaryGetter](const FAggregated& A, const FAggregated& B)
	{
		return PrimaryGetter(A) > PrimaryGetter(B);
	});

	TArray<FNiagaraStatsRowPtr> Rows;
	const int32 Count = FMath::Min(Sorted.Num(), 5);
	for (int32 i = 0; i < Count; ++i)
	{
		auto Row = MakeShared<FNiagaraStatsRow>();
		Row->Rank           = i + 1;
		Row->SystemName     = Sorted[i]->SystemName;
		Row->PrimaryValue   = PrimaryGetter(*Sorted[i]);
		Row->PrimaryUnit    = PrimaryUnit;
		Row->SecondaryValue = SecondaryGetter(*Sorted[i]);
		Row->SecondaryUnit  = SecondaryUnit;
		Rows.Add(MoveTemp(Row));
	}

	// Pad to 5 rows so the table always looks uniform.
	while (Rows.Num() < 5)
	{
		auto Row = MakeShared<FNiagaraStatsRow>();
		Row->Rank = Rows.Num() + 1;
		Row->SystemName = TEXT("—");
		Rows.Add(MoveTemp(Row));
	}

	return Rows;
}

// ---- UI construction helpers ---------------------------------------------------

TSharedRef<SWidget> SNiagaraRangeStatsView::MakeRangeHeaderBar()
{
	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.f, 0.f, 16.f, 0.f)
		[
			SAssignNew(RangeTextWidget, STextBlock)
			.Text(RangeText)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SAssignNew(FrameCountWidget, STextBlock)
			.Text(FrameCountText)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		];
}

TSharedRef<SWidget> SNiagaraRangeStatsView::MakeStatsSection(
	const FText&                 SectionTitle,
	const FLinearColor&          HeaderColor,
	const FText&                 PrimaryColumnLabel,
	const FText&                 SecondaryColumnLabel,
	TArray<FNiagaraStatsRowPtr>& Rows)
{
	// Create a list view pointer reference to store.
	TSharedPtr<SListView<FNiagaraStatsRowPtr>> LocalListView;

	TSharedRef<SHeaderRow> Header =
		SNew(SHeaderRow)
		+ SHeaderRow::Column(ColId_Rank)
			.DefaultLabel(LOCTEXT("Col_Rank", "#"))
			.FixedWidth(28.f)
		+ SHeaderRow::Column(ColId_System)
			.DefaultLabel(LOCTEXT("Col_System", "System"))
			.FillWidth(1.f)
		+ SHeaderRow::Column(ColId_Primary)
			.DefaultLabel(PrimaryColumnLabel)
			.FixedWidth(130.f)
		+ SHeaderRow::Column(ColId_Secondary)
			.DefaultLabel(SecondaryColumnLabel)
			.FixedWidth(130.f);

	TSharedRef<SListView<FNiagaraStatsRowPtr>> ListView =
		SNew(SListView<FNiagaraStatsRowPtr>)
		.ListItemsSource(&Rows)
		.HeaderRow(Header)
		.OnGenerateRow_Lambda(
			[this, PrimaryColumnLabel, SecondaryColumnLabel]
			(FNiagaraStatsRowPtr Item, const TSharedRef<STableViewBase>& OwnerTable)
			{
				return SNew(SNiagaraStatsTableRow, OwnerTable)
					.Row(Item)
					.PrimaryColumnLabel(PrimaryColumnLabel)
					.SecondaryColumnLabel(SecondaryColumnLabel);
			});

	LocalListView = ListView;

	// Store back into the appropriate member pointer based on which row array this is.
	// We use pointer comparison to identify the section.
	if      (&Rows == &Rows_InstanceCount) { ListView_InstanceCount = LocalListView; }
	else if (&Rows == &Rows_GTCostTotal)   { ListView_GTCostTotal   = LocalListView; }
	else if (&Rows == &Rows_GTCostPerInst) { ListView_GTCostPerInst = LocalListView; }
	else if (&Rows == &Rows_RTCostTotal)   { ListView_RTCostTotal   = LocalListView; }
	else if (&Rows == &Rows_RTCostPerInst) { ListView_RTCostPerInst = LocalListView; }
	else if (&Rows == &Rows_GpuCostTotal)  { ListView_GpuCostTotal  = LocalListView; }

	return SNew(SExpandableArea)
		.InitiallyCollapsed(false)
		.HeaderContent()
		[
			SNew(SBorder)
			.BorderBackgroundColor(HeaderColor)
			.Padding(FMargin(8.f, 4.f))
			[
				SNew(STextBlock)
				.Text(SectionTitle)
				.ColorAndOpacity(FLinearColor::White)
			]
		]
		.BodyContent()
		[
			ListView
		];
}

} // namespace UE::NiagaraInsights

#undef LOCTEXT_NAMESPACE
