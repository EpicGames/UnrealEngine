// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

namespace UE::NiagaraInsights
{

class FNiagaraTimingViewSession;

/** One row in a top-5 table. */
struct FNiagaraStatsRow
{
	int32   Rank           = 0;
	FString SystemName;
	float   PrimaryValue   = 0.f;   // The metric used for ranking (ms, count, …)
	FString PrimaryUnit;            // e.g. "ms", "instances", "ms/inst"
	float   SecondaryValue = 0.f;   // Supplementary context (instances, total ms, …)
	FString SecondaryUnit;
};

using FNiagaraStatsRowPtr = TSharedPtr<FNiagaraStatsRow>;

/**
 * Displays top-5 Niagara systems across six metric categories for the
 * time range selected in the Timing Profiler.
 *
 * Categories:
 *   1. Instance Count           — avg active instances (GT)
 *   2. Game Thread Cost         — total GT cost (ms)
 *   3. GT Cost / Instance       — avg GT ms per live instance
 *   4. Render Thread Cost       — total RT cost (ms)
 *   5. RT Cost / Instance       — avg RT ms per live instance
 *   6. GPU Cost                 — total GPU cost (ms)
 */
class SNiagaraRangeStatsView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraRangeStatsView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SNiagaraRangeStatsView();

private:
	// ---- Data ----------------------------------------------------------------

	/** Aggregated per-system data accumulated while scanning a range. */
	struct FAggregated
	{
		FString SystemName;

		// Game thread
		uint64  GTSamples        = 0;
		double  GTTotalInstances = 0.0;   // sum of NumInstances across GT frames
		double  GTTotalMs        = 0.0;   // total GT wall-clock ms

		// Render thread
		uint64  RTSamples        = 0;
		double  RTTotalInstances = 0.0;
		double  RTTotalMs        = 0.0;

		// GPU
		double  GpuTotalMs       = 0.0;
		double  GpuTotalInst     = 0.0;

		// Derived
		double AvgGTInstances()   const { return GTSamples  > 0 ? GTTotalInstances / GTSamples  : 0.0; }
		double GTCostPerInst()    const { return GTTotalInstances > 0 ? GTTotalMs / GTTotalInstances : 0.0; }
		double RTCostPerInst()    const { return RTTotalInstances > 0 ? RTTotalMs / RTTotalInstances : 0.0; }
		double GpuCostPerInst()   const { return GpuTotalInst     > 0 ? GpuTotalMs / GpuTotalInst  : 0.0; }
	};

	void RebuildStats(double StartTime, double EndTime);

	static TArray<FNiagaraStatsRowPtr> BuildTop5(
		const TArray<FAggregated>& Data,
		TFunctionRef<float(const FAggregated&)> PrimaryGetter,
		const FString& PrimaryUnit,
		TFunctionRef<float(const FAggregated&)> SecondaryGetter,
		const FString& SecondaryUnit);

	// ---- UI helpers ----------------------------------------------------------

	TSharedRef<SWidget> MakeRangeHeaderBar();

	/** Build one collapsible top-5 section. */
	TSharedRef<SWidget> MakeStatsSection(
		const FText&                       SectionTitle,
		const FLinearColor&                HeaderColor,
		const FText&                       PrimaryColumnLabel,
		const FText&                       SecondaryColumnLabel,
		TArray<FNiagaraStatsRowPtr>&       Rows);

	TSharedRef<ITableRow> OnGenerateRow(
		FNiagaraStatsRowPtr Item,
		const TSharedRef<STableViewBase>& OwnerTable,
		const FText& PrimaryColumnLabel,
		const FText& SecondaryColumnLabel);

	// ---- Session binding -------------------------------------------------------

	void TryBindToActiveSession();
	void OnNiagaraTimingSessionBegun(FNiagaraTimingViewSession& Session);
	void OnNiagaraTimingSessionEnded(FNiagaraTimingViewSession& Session);
	void OnTimingRangeChanged(double StartTime, double EndTime);
	void UnbindCurrentSession();

	FNiagaraTimingViewSession* BoundSession = nullptr;
	FDelegateHandle            RangeChangedHandle;

	// ---- State ---------------------------------------------------------------

	FText    RangeText;
	FText    FrameCountText;
	double   CurrentStartTime = 0.0;
	double   CurrentEndTime   = 0.0;

	// Six top-5 row arrays — rebuilt whenever the range changes.
	TArray<FNiagaraStatsRowPtr> Rows_InstanceCount;
	TArray<FNiagaraStatsRowPtr> Rows_GTCostTotal;
	TArray<FNiagaraStatsRowPtr> Rows_GTCostPerInst;
	TArray<FNiagaraStatsRowPtr> Rows_RTCostTotal;
	TArray<FNiagaraStatsRowPtr> Rows_RTCostPerInst;
	TArray<FNiagaraStatsRowPtr> Rows_GpuCostTotal;

	// List-view widgets so we can call RequestListRefresh().
	TSharedPtr<SListView<FNiagaraStatsRowPtr>> ListView_InstanceCount;
	TSharedPtr<SListView<FNiagaraStatsRowPtr>> ListView_GTCostTotal;
	TSharedPtr<SListView<FNiagaraStatsRowPtr>> ListView_GTCostPerInst;
	TSharedPtr<SListView<FNiagaraStatsRowPtr>> ListView_RTCostTotal;
	TSharedPtr<SListView<FNiagaraStatsRowPtr>> ListView_RTCostPerInst;
	TSharedPtr<SListView<FNiagaraStatsRowPtr>> ListView_GpuCostTotal;

	// Range header text widgets so we can update them in-place.
	TSharedPtr<STextBlock> RangeTextWidget;
	TSharedPtr<STextBlock> FrameCountWidget;
};

} // namespace UE::NiagaraInsights
